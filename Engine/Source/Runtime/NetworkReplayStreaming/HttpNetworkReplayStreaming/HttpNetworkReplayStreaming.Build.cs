// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HttpNetworkReplayStreaming : ModuleRules
    {
        public HttpNetworkReplayStreaming(ReadOnlyTargetRules Target) : base(Target)
        {
            ShortName = "HttpReplayStreaming";

            PrivateDependencyModuleNames.AddRange(
                new string[]
				{
					"Core",
					"CoreOnline",
					"CoreUObject",
                    "Engine",
                    "HTTP",
					"NetworkReplayStreaming",
					"Json",
				} );

            if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Mac)
            {
                AddEngineThirdPartyPrivateStaticDependencies(Target,"libWebSockets");
                AddEngineThirdPartyPrivateStaticDependencies(Target,"zlib");
            }

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
    }
}
