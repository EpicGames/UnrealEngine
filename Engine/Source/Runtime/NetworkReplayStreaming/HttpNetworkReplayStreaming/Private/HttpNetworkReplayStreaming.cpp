// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpNetworkReplayStreaming.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "HAL/LowLevelMemTracker.h"
#include "Engine/World.h"
#include "UnrealEngine.h"
#include "Engine/LocalPlayer.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "HttpManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "Misc/Fork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HttpNetworkReplayStreaming)

DEFINE_LOG_CATEGORY_STATIC( LogHttpReplay, Log, All );

DECLARE_STATS_GROUP( TEXT( "HttpReplay" ), STATGROUP_HttpReplay, STATCAT_Advanced )

DECLARE_DWORD_ACCUMULATOR_STAT( TEXT( "Http replay raw bytes" ), STAT_HttpReplay_Raw, STATGROUP_HttpReplay );
DECLARE_DWORD_ACCUMULATOR_STAT( TEXT( "Http replay compressed bytes" ), STAT_HttpReplay_Compressed, STATGROUP_HttpReplay );
DECLARE_CYCLE_STAT( TEXT( "Http replay compress time" ), STAT_HttpReplay_CompressTime, STATGROUP_HttpReplay );
DECLARE_CYCLE_STAT( TEXT( "Http replay decompress time" ), STAT_HttpReplay_DecompressTime, STATGROUP_HttpReplay );

static TAutoConsoleVariable<FString> CVarMetaFilterOverride( TEXT( "httpReplay.MetaFilterOverride" ), TEXT( "" ), TEXT( "" ) );
static TAutoConsoleVariable<float> CVarChunkUploadDelayInSeconds( TEXT( "httpReplay.ChunkUploadDelayInSeconds" ), 10.0f, TEXT( "" ) );
static TAutoConsoleVariable<int> CVarMaxCacheSize( TEXT( "httpReplay.MaxCacheSize" ), 1024 * 1024 * 10, TEXT( "" ) );		// 10 MB cache by default

static const FString GetUserStringFromUserIndex(const int32 UserIndex)
{
	if (UserIndex != INDEX_NONE)
	{
		if (UWorld* World = GWorld.GetReference())
		{
			if (GEngine != nullptr)
			{
				if (ULocalPlayer* LocalPlayer = GEngine->GetLocalPlayerFromControllerId(World, UserIndex))
				{
					return LocalPlayer->GetPreferredUniqueNetId().ToString();
				}
			}
		}
	}

	return FString();
}

class FNetworkReplayListItem : public FJsonSerializable
{
public:
	FNetworkReplayListItem() : SizeInBytes( 0 ), DemoTimeInMs( 0 ), NumViewers( 0 ), bIsLive( false ), Changelist( 0 ), bShouldKeep( false ) {}
	virtual ~FNetworkReplayListItem() {}

	FString		AppName;
	FString		SessionName;
	FString		FriendlyName;
	FDateTime	Timestamp;
	int32		SizeInBytes;
	int32		DemoTimeInMs;
	int32		NumViewers;
	bool		bIsLive;
	int32		Changelist;
	bool		bShouldKeep;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE( "AppName",		AppName );
		JSON_SERIALIZE( "SessionName",	SessionName );
		JSON_SERIALIZE( "FriendlyName",	FriendlyName );
		JSON_SERIALIZE( "Timestamp",		Timestamp );
		JSON_SERIALIZE( "SizeInBytes",	SizeInBytes );
		JSON_SERIALIZE( "DemoTimeInMs",	DemoTimeInMs );
		JSON_SERIALIZE( "NumViewers",	NumViewers );
		JSON_SERIALIZE( "bIsLive",		bIsLive );
		JSON_SERIALIZE( "Changelist",	Changelist );
		JSON_SERIALIZE( "shouldKeep",	bShouldKeep );
	END_JSON_SERIALIZER
};

class FNetworkReplayList : public FJsonSerializable
{
public:
	FNetworkReplayList()
	{}
	virtual ~FNetworkReplayList() {}

	TArray< FNetworkReplayListItem > Replays;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_ARRAY_SERIALIZABLE( "replays", Replays, FNetworkReplayListItem );
	END_JSON_SERIALIZER
};

class FNetworkReplayUserList : public FJsonSerializable
{
public:
	FNetworkReplayUserList()
	{
	}
	virtual ~FNetworkReplayUserList()
	{
	}

	TArray< FString > Users;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_ARRAY( "users", Users );
	END_JSON_SERIALIZER
};

class FNetworkReplayStartUploadingResponse : public FJsonSerializable
{
public:
	FNetworkReplayStartUploadingResponse()
	{}
	virtual ~FNetworkReplayStartUploadingResponse() {}

	FString SessionId;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("sessionId", SessionId);
	END_JSON_SERIALIZER
};

class FNetworkReplayStartDownloadingResponse : public FJsonSerializable
{
public:
	FNetworkReplayStartDownloadingResponse()
	{}
	virtual ~FNetworkReplayStartDownloadingResponse() {}

	FString State;
	FString Viewer;
	int32 Time;
	int32 NumChunks;
	FString Metadata;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("state", State);
		JSON_SERIALIZE("numChunks", NumChunks);
		JSON_SERIALIZE("time", Time);
		JSON_SERIALIZE("viewerId", Viewer);
		JSON_SERIALIZE("meta", Metadata);
	END_JSON_SERIALIZER
};

void FHttpStreamFArchive::Serialize( void* V, int64 Length ) 
{
	if (IsLoading())
	{
		if ((Pos + Length) > Buffer.Num())
		{
			SetError();
			return;
		}
			
		FMemory::Memcpy(V, Buffer.GetData() + Pos, Length);

		Pos += IntCastChecked<int32>(Length);
	}
	else
	{
		check(Pos <= Buffer.Num());

		const int32 LocalLength = IntCastChecked<int32>(Length);
		const int32 SpaceNeeded = LocalLength - (Buffer.Num() - Pos);

		if (SpaceNeeded > 0)
		{
			Buffer.AddZeroed(SpaceNeeded);
		}

		FMemory::Memcpy(Buffer.GetData() + Pos, V, LocalLength);

		Pos += LocalLength;
	}
}

int64 FHttpStreamFArchive::Tell() 
{
	return Pos;
}

int64 FHttpStreamFArchive::TotalSize()
{
	return Buffer.Num();
}

void FHttpStreamFArchive::Seek( int64 InPos ) 
{
	ensureMsgf( InPos >= 0 && InPos <= Buffer.Num(), TEXT("FHttpStreamFArchive::Seek invalid InPos %" INT64_FMT " Buffer Length %i"), InPos, Buffer.Num() );
	Pos = FMath::Clamp( static_cast<int32>(InPos), 0, Buffer.Num() );
}

bool FHttpStreamFArchive::AtEnd() 
{
	return Pos >= Buffer.Num() && bAtEndOfReplay;
}

FHttpNetworkReplayStreamer::FHttpNetworkReplayStreamer() : 
	StreamChunkIndex( 0 ), 
	LastChunkTime( 0 ), 
	LastRefreshViewerTime( 0 ),
	LastRefreshCheckpointTime( 0 ),
	StreamerState(EReplayStreamerState::Idle),
	bStopStreamingCalled( false ), 
	bStreamIsLive( false ),
	NumTotalStreamChunks( 0 ),
	TotalDemoTimeInMS( 0 ),
	StreamTimeRangeStart( 0 ),
	StreamTimeRangeEnd( 0 ),
	HighPriorityEndTime( 0 ),
	DownloadCheckpointIndex( -1 ),
	DeltaDownloadCheckpointIndex( -1 ),
	LastGotoTimeInMS( -1 ),
	TotalUploadBytes( 0 ),
	RefreshViewerFails( 0 )
{
	// Initialize the server URL
	GConfig->GetString( TEXT( "HttpNetworkReplayStreaming" ), TEXT( "ServerURL" ), ServerURL, GEngineIni );
}

FString FHttpNetworkReplayStreamer::GetRecordingMetadata() const
{
	FString MetaString;
	FParse::Value( FCommandLine::Get(), TEXT( "ReplayMeta=" ), MetaString );
	return MetaString;
}

void FHttpNetworkReplayStreamer::StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate)
{
	FStartStreamingResult StreamingResult;
	StreamingResult.bRecording = Params.bRecord;

	if ( !SessionName.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StartStreaming. SessionName already set." );
		Delegate.ExecuteIfBound(StreamingResult);
		return;
	}

	if ( IsStreaming() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StartStreaming. IsStreaming == true." );
		Delegate.ExecuteIfBound(StreamingResult);
		return;
	}

	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StartStreaming. ServerURL is empty, aborting." );
		Delegate.ExecuteIfBound(StreamingResult);
		return;
	}

	if ( IsHttpRequestInFlight() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StartStreaming. IsHttpRequestInFlight == true." );
		Delegate.ExecuteIfBound(StreamingResult);
		return;
	}

	ReplayVersion = Params.ReplayVersion;

	// Remember the delegate, which we'll call as soon as the header is available
	StartStreamingDelegate = Delegate;

	// Setup the archives
	StreamArchive.SetIsLoading(!Params.bRecord);
	StreamArchive.SetIsSaving(Params.bRecord);
	StreamArchive.bAtEndOfReplay	= false;

	HeaderArchive.SetIsLoading(!Params.bRecord);
	HeaderArchive.SetIsSaving(Params.bRecord);

	CheckpointArchive.SetIsLoading(!Params.bRecord);
	CheckpointArchive.SetIsSaving(Params.bRecord);

	TotalDemoTimeInMS = 0;

	StreamTimeRangeStart	= 0;
	StreamTimeRangeEnd		= 0;

	EventGroupSet.Empty();

	// Create the Http request and add to pending request list
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	StreamChunkIndex = 0;

	TotalUploadBytes = 0;

	RefreshViewerFails = 0;

	if ( !Params.bRecord )
	{
		// We are streaming down
		StreamerState = EReplayStreamerState::Playback;

		SessionName = Params.CustomName;

		FString UserName;

		if (Params.UserIndices.Num() == 1)
		{
			UserName = GetUserStringFromUserIndex(Params.UserIndices[0]);
		}

		// Notify the http server that we want to start downloading a replay
		const FString URL = FString::Printf( TEXT( "%sreplay/%s/startDownloading?user=%s" ), *ServerURL, *SessionName, *UserName );

		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::StartStreaming. URL: %ls", *URL );

		HttpRequest->SetURL( URL );
		HttpRequest->SetVerb( TEXT( "POST" ) );

		HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpStartDownloadingFinished );
	
		// Add the request to start downloading
		AddRequestToQueue( EQueuedHttpRequestType::StartDownloading, HttpRequest );

		// Download the header (will add to queue)
		DownloadHeader(FDownloadHeaderCallback());

		// Download the first set of checkpoints (will add to queue)
		EnumerateCheckpoints();
	}
	else
	{
		// We are streaming up
		StreamerState = EReplayStreamerState::Recording;

		SessionName.Empty();

		FString URL;

		if ( !Params.CustomName.IsEmpty() )
		{
			const FString SessionIDOverride = Params.CustomName.ToLower();

			URL = FString::Printf( TEXT( "%sreplay/%s?app=%s&version=%u&cl=%u&friendlyName=%s" ), *ServerURL, *SessionIDOverride, *ReplayVersion.AppString, ReplayVersion.NetworkVersion, ReplayVersion.Changelist, *FGenericPlatformHttp::UrlEncode( Params.FriendlyName ) );
		}
		else
		{
			URL = FString::Printf( TEXT( "%sreplay?app=%s&version=%u&cl=%u&friendlyName=%s" ), *ServerURL, *ReplayVersion.AppString, ReplayVersion.NetworkVersion, ReplayVersion.Changelist, *FGenericPlatformHttp::UrlEncode( Params.FriendlyName ) );
		}

		FString MetaString = GetRecordingMetadata();
		if ( !MetaString.IsEmpty() )
		{
			URL += FString::Printf( TEXT( "&meta=%s" ), *FGenericPlatformHttp::UrlEncode( MetaString ) );
		}

		// Notify the http server that we want to start uploading a replay
		HttpRequest->SetURL( URL );

		HttpRequest->SetVerb( TEXT( "POST" ) );

		HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpStartUploadingFinished );

		HttpRequest->SetHeader( TEXT( "Content-Type" ), TEXT( "application/json" ) );

		if (Params.UserIndices.Num() > 0)
		{
			FNetworkReplayUserList UserList;

			for (int32 UserIdx : Params.UserIndices)
			{
				UserList.Users.Add(GetUserStringFromUserIndex(UserIdx));
			}

			HttpRequest->SetContentAsString( UserList.ToJson() );
		}

		AddRequestToQueue( EQueuedHttpRequestType::StartUploading, HttpRequest, 3, 2.0f );
		
		// We need to upload the header AFTER StartUploading is done (so we have session name)
		RefreshHeader();
	}
}

void FHttpNetworkReplayStreamer::RefreshHeader()
{
	AddRequestToQueue(EQueuedHttpRequestType::UploadHeader, nullptr);
}

void FHttpNetworkReplayStreamer::AddRequestToQueue( const EQueuedHttpRequestType::Type Type, TSharedPtr<class IHttpRequest, ESPMode::ThreadSafe> Request, const int32 InMaxRetries, const float InRetryDelay )
{
	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::AddRequestToQueue. Type: %ls", EQueuedHttpRequestType::ToString( Type ) );

	QueuedHttpRequests.Add( TSharedPtr< FQueuedHttpRequest >( new FQueuedHttpRequest( Type, Request, InMaxRetries, InRetryDelay ) ) );
}

void FHttpNetworkReplayStreamer::AddCustomRequestToQueue( TSharedPtr< FQueuedHttpRequest > Request )
{
	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::AddCustomRequestToQueue. Type: %ls", EQueuedHttpRequestType::ToString( Request->Type ) );

	QueuedHttpRequests.Add( Request );
}

void FHttpNetworkReplayStreamer::AddResponseToCache( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse )
{
	if ( !HttpRequest.IsValid() )
	{
		return;
	}

	if ( !HttpResponse.IsValid() )
	{
		return;
	}

	// Add to cache (or freshen existing entry)
	ResponseCache.Add( HttpRequest->GetURL(), FCachedResponse( HttpResponse, FPlatformTime::Seconds() ) );

	// Anytime we add something to cache, make sure it's within budget
	CleanupResponseCache();
}

void FHttpNetworkReplayStreamer::CleanupResponseCache()
{
	// Remove older entries until we're under the CVarMaxCacheSize threshold
	while ( ResponseCache.Num() )
	{
		double OldestTime = 0.0;
		FString OldestKey;
		uint32 TotalSize = 0;

		for ( auto It = ResponseCache.CreateIterator(); It; ++It )
		{
			if ( OldestKey.IsEmpty() || It.Value().LastAccessTime < OldestTime )
			{
				OldestTime = It.Value().LastAccessTime;
				OldestKey = It.Key();
			}

			// Accumulate total cache size
			TotalSize += It.Value().Response->GetContent().Num();
		}

		check( !OldestKey.IsEmpty() );

		const uint32 MaxCacheSize = CVarMaxCacheSize.GetValueOnGameThread();

		if ( TotalSize <= MaxCacheSize )
		{
			break;	// We're good
		}

		ResponseCache.Remove( OldestKey );
	}
}

static FString BuildRequestErrorString( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse )
{
	FString ExtraInfo;

	if ( HttpRequest.IsValid() )
	{
		ExtraInfo += FString::Printf( TEXT( "URL: %s" ), *HttpRequest->GetURL() );
		ExtraInfo += FString::Printf( TEXT( ", Verb: %s" ), *HttpRequest->GetVerb() );

		const TArray< FString > AllHeaders = HttpRequest->GetAllHeaders();

		for ( int i = 0; i < AllHeaders.Num(); i++ )
		{
			ExtraInfo += TEXT( ", " );
			ExtraInfo += AllHeaders[i];
		}
	}
	else
	{
		ExtraInfo = TEXT( "HttpRequest NULL." );
	}

	return FString::Printf( TEXT( "Response code: %d, Extra info: %s" ), HttpResponse.IsValid() ? HttpResponse->GetResponseCode() : 0, *ExtraInfo );
}

bool FHttpNetworkReplayStreamer::RetryRequest( TSharedPtr< FQueuedHttpRequest > Request, FHttpResponsePtr HttpResponse, const bool bIgnoreResponseCode )
{
	if ( !Request.IsValid() )
	{
		return false;
	}

	if ( Request->MaxRetries == 0 || Request->RetryProgress >= Request->MaxRetries )
	{
		return false;
	}
	
	if ( !HttpResponse.IsValid() )
	{
		return false;
	}

	if ( !bIgnoreResponseCode )
	{
		if ( HttpResponse->GetResponseCode() < 500 || HttpResponse->GetResponseCode() >= 600 )
		{
			return false;		// Only retry on 5xx return codes
		}
	}

	Request->RetryProgress++;

	if ( Request->RetryDelay == 0.0f )
	{
		// Re-process it immediately if it's not on a delay
		Request->Request->ProcessRequest();
	}
	else
	{
		Request->NextRetryTime = FPlatformTime::Seconds() + Request->RetryDelay;
	}

	// Put request back on InFlightRequest
	InFlightHttpRequest = Request;

	UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::RetryRequest. Retrying: %i, %ls", Request->RetryProgress, *BuildRequestErrorString( Request->Request, HttpResponse ) );

	return true;
}

void FHttpNetworkReplayStreamer::StopStreaming()
{
	if ( StartStreamingDelegate.IsBound() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StopStreaming. Called while existing StartStreaming request wasn't finished" );
		CancelStreamingRequests();
		check( !IsStreaming() );
		return;
	}

	if ( !IsStreaming() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StopStreaming. Not currently streaming." );
		check( bStopStreamingCalled == false );
		return;
	}

	if ( bStopStreamingCalled )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::StopStreaming. Already called" );
		return;
	}

	bStopStreamingCalled = true;

	if ( StreamerState == EReplayStreamerState::Playback )
	{
		// Refresh one last time, pasing in bFinal == true, this will notify the server we're done (remove us from viewers list)
		RefreshViewer( true );
	}
	else if ( StreamerState == EReplayStreamerState::Recording )
	{
		// Flush any final pending stream
		FlushStream();
		// Send one last http request to stop uploading		
		StopUploading();
	}

	// Finally, add the stop streaming request, which should put things in the right state after the above requests are done
	AddRequestToQueue( EQueuedHttpRequestType::StopStreaming, nullptr );
}

void FHttpNetworkReplayStreamer::UploadHeader()
{
	check( StreamArchive.IsSaving() );

	if ( SessionName.IsEmpty() )
	{
		// IF there is no active session, or we are not recording, we don't need to flush
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::UploadHeader. No session name!" );
		return;
	}

	if ( HeaderArchive.Buffer.Num() == 0 )
	{
		// Header wasn't serialized
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::UploadHeader. No header to upload" );
		return;
	}

	if ( !IsStreaming() )
	{
		UE_LOGF(LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::UploadHeader. Not currently streaming");
		return;
	}

	// First upload the header
	UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::UploadHeader. Header. StreamChunkIndex: %i, Size: %i", StreamChunkIndex, HeaderArchive.Buffer.Num() );

	// Create the Http request and add to pending request list
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpHeaderUploadFinished );

	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/file/replay.header?numChunks=%i&time=%i" ), *ServerURL, *SessionName, StreamChunkIndex, TotalDemoTimeInMS ) );
	HttpRequest->SetVerb( TEXT( "POST" ) );
	HttpRequest->SetHeader( TEXT( "Content-Type" ), TEXT( "application/octet-stream" ) );
	HttpRequest->SetContent( HeaderArchive.Buffer );

	// We're done with the header archive
	HeaderArchive.Buffer.Empty();
	HeaderArchive.Pos = 0;

	AddRequestToQueue( EQueuedHttpRequestType::UploadingHeader, HttpRequest, 3, 2.0f );

	LastChunkTime = FPlatformTime::Seconds();
}

bool FHttpNetworkReplayStreamer::CompressRequest(FHttpRequestPtr HttpRequest, const TArray<uint8>& RequestBuffer) const
{
	if ( SupportsCompression() )
	{
		SCOPE_CYCLE_COUNTER( STAT_HttpReplay_CompressTime );

		const double StartTime = FPlatformTime::Seconds();
		FHttpStreamFArchive Compressed;

		if ( !CompressBuffer( RequestBuffer, Compressed ) )
		{
			return false;
		}

		const double EndTime = FPlatformTime::Seconds();

		UE_LOGF( LogHttpReplay, VeryVerbose, "Compressed request. Original: %i, Compressed: %i, Time: %2.2f MS", StreamArchive.Buffer.Num(), Compressed.Buffer.Num(), ( EndTime - StartTime ) * 1000.0f );

		HttpRequest->SetContent( Compressed.Buffer );
	}
	else
	{
		HttpRequest->SetContent( RequestBuffer );
	}

	return true;
}

void FHttpNetworkReplayStreamer::FlushStream()
{
	check( StreamArchive.IsSaving() );

	if ( SessionName.IsEmpty() )
	{
		// If we haven't uploaded the header, or we are not recording, we don't need to flush
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::FlushStream. Waiting on header upload." );
		return;
	}

	if ( StreamArchive.Buffer.Num() == 0 )
	{
		// Nothing to flush
		return;
	}

	// Upload any new streamed data to the http server
	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::FlushStream. StreamChunkIndex: %i, Size: %i", StreamChunkIndex, StreamArchive.Buffer.Num() );

	// Create the Http request and add to pending request list
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpUploadStreamFinished );

	if (!CompressRequest(HttpRequest, StreamArchive.Buffer))
	{
		SetLastError(EHttpReplayResult::CompressionFailed);
		return;
	}

	TotalUploadBytes += HttpRequest->GetContentLength();

	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/file/stream.%i?numChunks=%i&time=%i&mTime1=%i&mTime2=%i&absSize=%llu" ), *ServerURL, *SessionName, StreamChunkIndex, StreamChunkIndex + 1, TotalDemoTimeInMS, StreamTimeRangeStart, StreamTimeRangeEnd, TotalUploadBytes ) );
	HttpRequest->SetVerb( TEXT( "POST" ) );
	HttpRequest->SetHeader( TEXT( "Content-Type" ), TEXT( "application/octet-stream" ) );

	StreamArchive.Buffer.Empty();
	StreamArchive.Pos = 0;

	// Keep track of the time range we have in our buffer, so we can accurately upload that each time we submit a chunk
	StreamTimeRangeStart = StreamTimeRangeEnd;

	StreamChunkIndex++;

	AddRequestToQueue( EQueuedHttpRequestType::UploadingStream, HttpRequest, 2, 2.0f );

	LastChunkTime = FPlatformTime::Seconds();
}

void FHttpNetworkReplayStreamer::ConditionallyFlushStream()
{
	if ( IsHttpRequestInFlight() || HasPendingHttpRequests() )
	{
		return;
	}
	
	const float FLUSH_TIME_IN_SECONDS = CVarChunkUploadDelayInSeconds.GetValueOnGameThread();

	if ( FPlatformTime::Seconds() - LastChunkTime > FLUSH_TIME_IN_SECONDS )
	{
		FlushStream();
	}
};

void FHttpNetworkReplayStreamer::StopUploading()
{
	// Create the Http request and add to pending request list
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpStopUploadingFinished );

	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/stopUploading?numChunks=%i&time=%i&absSize=%llu" ), *ServerURL, *SessionName, StreamChunkIndex, TotalDemoTimeInMS, TotalUploadBytes ) );
	HttpRequest->SetVerb( TEXT( "POST" ) );
	HttpRequest->SetHeader( TEXT( "Content-Type" ), TEXT( "application/octet-stream" ) );

	AddRequestToQueue( EQueuedHttpRequestType::StopUploading, HttpRequest, 3, 2.0f );
};

void FHttpNetworkReplayStreamer::FlushCheckpoint( const uint32 TimeInMS )
{
	if ( CheckpointArchive.Buffer.Num() == 0 )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::FlushCheckpoint. Checkpoint is empty." );
		return;
	}
	
	// Flush any existing stream, we need checkpoints to line up with the next chunk
	FlushStream();

	// Flush the checkpoint
	FlushCheckpointInternal( TimeInMS );
}

FQueuedGotoFakeCheckpoint::FQueuedGotoFakeCheckpoint() : FQueuedHttpRequest( EQueuedHttpRequestType::UploadingCustomEvent, nullptr )
{
}

bool FQueuedGotoFakeCheckpoint::PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName )
{
	// Make sure to reset the checkpoint archive (this is how we signify that the engine should start from the beginning of the steam (we don't need a checkpoint for that))
	Streamer->CheckpointArchive.Buffer.Empty();
	Streamer->CheckpointArchive.Pos = 0;

	if (!Streamer->IsDataAvailableForTimeRange(0, IntCastChecked<uint32>(Streamer->LastGotoTimeInMS)))
	{
		// Completely reset our stream (we're going to start downloading from the start of the checkpoint)
		Streamer->StreamArchive.Buffer.Empty();

		// Reset our stream range
		Streamer->StreamTimeRangeStart	= 0;
		Streamer->StreamTimeRangeEnd	= 0;

		// Reset chunk index
		Streamer->StreamChunkIndex		= 0;
	}

	Streamer->StreamArchive.Pos				= 0;
	Streamer->StreamArchive.bAtEndOfReplay	= false;

	FGotoCallback& GotoCheckpointDelegate = Streamer->GotoCheckpointDelegate;
	if (GotoCheckpointDelegate.IsBound())
	{
		FGotoResult Result;
		Result.Result = EStreamingOperationResult::Success;
		Result.ExtraTimeMS = Streamer->LastGotoTimeInMS;
		Result.CheckpointInfo.CheckpointIndex = FReplayCheckpointInfo::NO_CHECKPOINT;
		Result.CheckpointInfo.CheckpointStartTime = FReplayCheckpointInfo::NO_CHECKPOINT;

		GotoCheckpointDelegate.Execute( Result );
		GotoCheckpointDelegate = FGotoCallback();
	}	

	Streamer->LastGotoTimeInMS = -1;

	return true;
}

void FHttpNetworkReplayStreamer::GotoCheckpointIndex( const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType )
{
	if ( !IsCheckpointTypeSupported(CheckpointType) )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::GotoCheckpointIndex. Unsupported checkpoint type." );
		Delegate.ExecuteIfBound(FGotoResult());
		return;
	}

	if ( GotoCheckpointDelegate.IsBound() || DownloadCheckpointIndex != -1 || DeltaDownloadCheckpointIndex != -1 )
	{
		// If we're currently going to a checkpoint now, ignore this request
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::GotoCheckpointIndex. Busy processing another checkpoint." );
		Delegate.ExecuteIfBound(FGotoResult());
		return;
	}

	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::GotoCheckpointIndex. ServerURL is empty, aborting." );
		Delegate.ExecuteIfBound(FGotoResult());
		return;
	}

	if ( CheckpointIndex == -1 )
	{
		GotoCheckpointDelegate = Delegate;
		SetHighPriorityTimeRange(0, IntCastChecked<uint32>(LastGotoTimeInMS));
		LastChunkTime = 0;		// Force the next chunk to start downloading immediately
		AddCustomRequestToQueue( TSharedPtr< FQueuedHttpRequest >( new FQueuedGotoFakeCheckpoint() ) );
		return;
	}

	if ( !CheckpointList.ReplayEvents.IsValidIndex( CheckpointIndex ) )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::GotoCheckpointIndex. Invalid checkpoint index." );
		Delegate.ExecuteIfBound(FGotoResult());
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Download the next stream chunk
	HttpRequest->SetVerb( TEXT( "GET" ) );

	if (CheckpointType == EReplayCheckpointType::Delta)
	{
		DeltaDownloadCheckpointIndex = 0;

		HttpRequest->SetURL( FString::Printf( TEXT( "%sevent/%s" ), *ServerURL,  *CheckpointList.ReplayEvents[DeltaDownloadCheckpointIndex].ID ) );

		HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished );
	}
	else
	{
		HttpRequest->SetURL( FString::Printf( TEXT( "%sevent/%s" ), *ServerURL,  *CheckpointList.ReplayEvents[CheckpointIndex].ID ) );

		HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished );
	}

	GotoCheckpointDelegate	= Delegate;
	DownloadCheckpointIndex = CheckpointIndex;

	AddRequestToQueue( EQueuedHttpRequestType::DownloadingCheckpoint, HttpRequest );
}

void FHttpNetworkReplayStreamer::SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate)
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::SearchEvents. ServerURL is empty, aborting." );
		Delegate.Execute(FSearchEventsResult());
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->SetURL(FString::Printf(TEXT("%sevent?group=%s"), *ServerURL, *EventGroup));
	HttpRequest->SetVerb(TEXT("GET"));

	FEnumerateStreamsCallback WrappedDelegate = FEnumerateStreamsCallback::CreateLambda([Delegate](const FEnumerateStreamsResult& Result)
	{
		if (Delegate.IsBound())
		{
			FSearchEventsResult NewResult;
			NewResult.Result = Result.Result;
			NewResult.CorruptedStreams = Result.CorruptedStreams;
			NewResult.FoundStreams = Result.FoundStreams;
			Delegate.Execute(NewResult);
		}
	});

	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished, WrappedDelegate);

	AddRequestToQueue(EQueuedHttpRequestType::EnumeratingSessions, HttpRequest);
}

void FHttpNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	RequestEventData(ReplayName, EventID, Delegate);
}

void FHttpNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	UE_LOGF(LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::RequestEventData: Ignoring explicit ReplayName.");
	RequestEventData(EventID, Delegate);
}

void FHttpNetworkReplayStreamer::RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::RequestEventData. ServerURL is empty, aborting." );
		Delegate.Execute(FRequestEventDataResult());
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Download the next stream chunk
	HttpRequest->SetURL(FString::Printf(TEXT("%sevent/%s"), *ServerURL, *EventID));
	HttpRequest->SetVerb(TEXT("GET"));

	HttpRequest->OnProcessRequestComplete().BindRaw(this, &FHttpNetworkReplayStreamer::HttpRequestEventDataFinished, Delegate);
	
	AddRequestToQueue(EQueuedHttpRequestType::RequestEventData, HttpRequest);
}

void FHttpNetworkReplayStreamer::HttpRequestEventDataFinished(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FRequestEventDataCallback RequestEventDataCompleteDelegate)
{
	RequestFinished(StreamerState, EQueuedHttpRequestType::RequestEventData, HttpRequest);

	FRequestEventDataResult Result;
	if (bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok)
	{
		UE_LOGF(LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpRequestEventDataFinished.");

		Result.ReplayEventListItem = HttpResponse->GetContent();
		Result.Result = EStreamingOperationResult::Success;
	}
	else
	{
		// FAIL
		UE_LOGF(LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpRequestEventDataFinished. FAILED, Response code: %d", HttpResponse.IsValid() ? HttpResponse->GetResponseCode() : 0);
	}

	RequestEventDataCompleteDelegate.ExecuteIfBound(Result);
}

void FHttpNetworkReplayStreamer::RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	UE_LOGF(LogHttpReplay, Log, "FHttpNetworkReplayStreamer::RequestEventGroupData is currently unsupported.");
	FRequestEventGroupDataResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FHttpNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(Group, Delegate);
}

void FHttpNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(Group, Delegate);
}

void FHttpNetworkReplayStreamer::GotoTimeInMS( const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType )
{
	if ( !IsCheckpointTypeSupported(CheckpointType) )
	{
		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::GotoTimeInMS. Unsupported checkpoint type." );
		Delegate.ExecuteIfBound( FGotoResult() );
		return;
	}

	if ( LastGotoTimeInMS != -1 || DownloadCheckpointIndex != -1 )
	{
		// If we're processing requests, be on the safe side and cancel the scrub
		// FIXME: We can cancel the in-flight requests as well
		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::GotoTimeInMS. Busy processing pending requests." );
		Delegate.ExecuteIfBound( FGotoResult() );
		return;
	}

	if ( GotoCheckpointDelegate.IsBound() )
	{
		// If we're currently going to a checkpoint now, ignore this request
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::GotoTimeInMS. Busy processing another checkpoint." );
		Delegate.ExecuteIfBound( FGotoResult() );
		return;
	}

	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::GotoTimeInMS. TimeInMS: %i", (int)TimeInMS );

	check( DownloadCheckpointIndex == -1 );
	check( LastGotoTimeInMS == -1 );

	int32 CheckpointIndex = -1;

	LastGotoTimeInMS = FMath::Min( TimeInMS, TotalDemoTimeInMS );

	if ( CheckpointList.ReplayEvents.Num() > 0 && TimeInMS >= CheckpointList.ReplayEvents[ CheckpointList.ReplayEvents.Num() - 1 ].Time1 )
	{
		// If we're after the very last checkpoint, that's the one we want
		CheckpointIndex = CheckpointList.ReplayEvents.Num() - 1;
	}
	else
	{
		// Checkpoints should be sorted by time, return the checkpoint that exists right before the current time
		// For fine scrubbing, we'll fast forward the rest of the way
		// NOTE - If we're right before the very first checkpoint, we'll return -1, which is what we want when we want to start from the very beginning
		for ( int i = 0; i < CheckpointList.ReplayEvents.Num(); i++ )
		{
			if ( TimeInMS < CheckpointList.ReplayEvents[i].Time1 )
			{
				CheckpointIndex = i - 1;
				break;
			}
		}
	}

	GotoCheckpointIndex(CheckpointIndex, Delegate, CheckpointType);
}

void FHttpNetworkReplayStreamer::FlushCheckpointInternal( uint32 TimeInMS )
{
	if ( SessionName.IsEmpty() || StreamerState != EReplayStreamerState::Recording || CheckpointArchive.Buffer.Num() == 0 )
	{
		// If there is no active session, or we are not recording, we don't need to flush
		CheckpointArchive.Buffer.Empty();
		CheckpointArchive.Pos = 0;
		return;
	}

	double Duration = 0;

	{
		FScopedDurationTimer Timer(Duration);

		// Create the Http request and add to pending request list
		TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

		HttpRequest->OnProcessRequestComplete().BindRaw(this, &FHttpNetworkReplayStreamer::HttpUploadCheckpointFinished);

		HttpRequest->SetURL(FString::Printf(TEXT("%sreplay/%s/event?group=checkpoint&time1=%i&time2=%i&meta=%i&incrementSize=false"), *ServerURL, *SessionName, StreamTimeRangeEnd, StreamTimeRangeEnd, StreamChunkIndex));
		HttpRequest->SetVerb(TEXT("POST"));
		HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/octet-stream"));

		if (!CompressRequest(HttpRequest, CheckpointArchive.Buffer))
		{
			SetLastError(EHttpReplayResult::CompressionFailed);
			return;
		}

		AddRequestToQueue(EQueuedHttpRequestType::UploadingCheckpoint, HttpRequest, 2, 2.0f);
	}

	// Upload any new streamed data to the http server
	UE_LOGF(LogHttpReplay, Log, "FHttpNetworkReplayStreamer::FlushCheckpointInternal. Size: %i, StreamChunkIndex: %i, Time: %2.2f MS", 
		CheckpointArchive.Buffer.Num(), 
		StreamChunkIndex,
		Duration * 1000.0f);

	CheckpointArchive.Buffer.Empty();
	CheckpointArchive.Pos = 0;	
}

FQueuedHttpRequestAddEvent::FQueuedHttpRequestAddEvent( const FString& InName, const uint32 InTimeInMS, const FString& InGroup, const FString& InMeta, const TArray<uint8>& InData, TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> InHttpRequest ) : FQueuedHttpRequest( EQueuedHttpRequestType::UploadingCustomEvent, InHttpRequest )
{
	Request->SetVerb( TEXT( "POST" ) );
	Request->SetHeader( TEXT( "Content-Type" ), TEXT( "application/octet-stream" ) );
	Request->SetContent( InData );

	Name		= InName;
	TimeInMS	= InTimeInMS;
	Group		= InGroup;
	Meta		= InMeta;
}

bool FQueuedHttpRequestAddEvent::PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName )
{
	if ( SessionName.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FQueuedHttpRequestAddEvent::PreProcess. SessionName is empty." );
		return false;
	}

	//
	// Now that we have the session name, we can set the URL
	//

	if ( !Name.IsEmpty() )
	{
		// Add or update existing event
		const FString EventName = SessionName + TEXT( "_" ) + Name;
		Request->SetURL( FString::Printf( TEXT( "%sreplay/%s/event/%s?group=%s&time1=%i&time2=%i&meta=%s&incrementSize=false" ), *ServerURL, *SessionName, *EventName, *Group, TimeInMS, TimeInMS, *FGenericPlatformHttp::UrlEncode( Meta ) ) );
	}
	else
	{
		Request->SetURL( FString::Printf( TEXT( "%sreplay/%s/event?group=%s&time1=%i&time2=%i&meta=%s&incrementSize=false" ), *ServerURL, *SessionName, *Group, TimeInMS, TimeInMS, *FGenericPlatformHttp::UrlEncode( Meta ) ) );
	}

	return true;
}

void FHttpNetworkReplayStreamer::AddOrUpdateEvent( const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data )
{
	if ( StreamerState != EReplayStreamerState::Recording && StreamerState != EReplayStreamerState::Playback )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::AddOrUpdateEvent. Not streaming." );
		return;
	}

	// Upload a custom event to replay server
	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::AddEvent. Size: %i, StreamChunkIndex: %i", Data.Num(), StreamChunkIndex );

	// Create the Http request and add to pending request list
	TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpUploadCustomEventFinished );

	// Add it as a custom event so we can snag the session name at the time of send (which we should have by then)
	AddCustomRequestToQueue( TSharedPtr< FQueuedHttpRequest >( new FQueuedHttpRequestAddEvent( Name, TimeInMS, Group, Meta, Data, HttpRequest ) ) );
}

void FHttpNetworkReplayStreamer::AddEvent( const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data )
{
	if (StreamerState != EReplayStreamerState::Recording && StreamerState != EReplayStreamerState::Playback)
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::AddEvent. Not streaming." );
		return;
	}

	if ( StreamerState == EReplayStreamerState::Recording && !EventGroupSet.Contains( Group ) )
	{
		// Add the group as a user, so we can quickly find replays that have these event types in them
		EventGroupSet.Add( Group );
		AddUserToReplay( Group );
	}

	AddOrUpdateEvent( TEXT( "" ), TimeInMS, Group, Meta, Data );
}

void FHttpNetworkReplayStreamer::DownloadHeader(const FDownloadHeaderCallback& Delegate)
{
	// Download header first
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/file/replay.header" ), *ServerURL, *SessionName ) );
	HttpRequest->SetVerb( TEXT( "GET" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpDownloadHeaderFinished, Delegate );

	AddRequestToQueue( EQueuedHttpRequestType::DownloadingHeader, HttpRequest );
}

bool FHttpNetworkReplayStreamer::IsTaskPendingOrInFlight( const EQueuedHttpRequestType::Type Type ) const
{
	for ( const TSharedPtr< FQueuedHttpRequest >& Request : QueuedHttpRequests )
	{
		if ( Request->Type == Type )
		{
			return true;
		}
	}

	if ( InFlightHttpRequest.IsValid() )
	{
		if ( InFlightHttpRequest->Type == Type )
		{
			return true;
		}
	}

	return false;
}

void FHttpNetworkReplayStreamer::CancelInFlightOrPendingTask( const EQueuedHttpRequestType::Type Type )
{
	for ( int i = QueuedHttpRequests.Num() - 1; i >= 0; i-- )
	{
		if ( QueuedHttpRequests[i]->Type == Type )
		{
			QueuedHttpRequests.RemoveAt( i );
		}
	}

	if ( InFlightHttpRequest.IsValid() && InFlightHttpRequest->Type == Type )
	{
		if ( InFlightHttpRequest->Request->OnProcessRequestComplete().IsBound() )
		{
			InFlightHttpRequest->Request->OnProcessRequestComplete().Unbind();
		}

		InFlightHttpRequest->Request->CancelRequest();
		InFlightHttpRequest = NULL;
	}
}

void FHttpNetworkReplayStreamer::ConditionallyDownloadNextChunk()
{
	if ( GotoCheckpointDelegate.IsBound() )
	{
		// Don't download stream chunk while we're waiting for a checkpoint to download
		return;
	}

	if ( IsTaskPendingOrInFlight( EQueuedHttpRequestType::DownloadingStream ) )
	{
		// Only download one chunk at a time
		return;
	}

	const bool bMoreChunksDefinitelyAvailable = StreamChunkIndex < NumTotalStreamChunks;		// We know for a fact there are more chunks available

	if ( !bMoreChunksDefinitelyAvailable && !bStreamIsLive )
	{
		// If we're not live, and there are no more chunks available, we don't need to go any further
		return;
	}

	// Determine if it's time to download the next chunk
	const bool bHighPriorityMode			= ( HighPriorityEndTime > 0 && StreamTimeRangeEnd < HighPriorityEndTime );			// We're within the high priority time range
	const bool bReallyNeedToDownloadChunk	= bHighPriorityMode && bMoreChunksDefinitelyAvailable;

	// If it's not critical to download the next chunk (i.e. we're not scrubbing or at the end already), then check to see if we should grab the next chunk
	if ( !bReallyNeedToDownloadChunk )
	{
		const double MIN_WAIT_FOR_NEXT_CHUNK_IN_SECONDS = 5;
		const double MAX_WAIT_FOR_NEXT_CHUNK_IN_SECONDS = 30;

		const double DownloadElapsedTime = FPlatformTime::Seconds() - LastChunkTime;

		if ( DownloadElapsedTime < MIN_WAIT_FOR_NEXT_CHUNK_IN_SECONDS )
		{
			return;		// Unless it's critical (i.e. bReallyNeedToDownloadChunk is true), never try faster than MIN_WAIT_FOR_NEXT_CHUNK_IN_SECONDS
		}

		if ( DownloadElapsedTime < MAX_WAIT_FOR_NEXT_CHUNK_IN_SECONDS && StreamTimeRangeEnd > StreamTimeRangeStart && StreamArchive.Buffer.Num() > 0 )
		{
			// Make a guess on how far we're in
			const float PercentIn		= StreamArchive.Buffer.Num() > 0 ? ( float )StreamArchive.Pos / ( float )StreamArchive.Buffer.Num() : 0.0f;
			const float TotalStreamTime	= ( float )( StreamTimeRangeEnd - StreamTimeRangeStart ) / 1000.0f;
			const float CurrentTime		= TotalStreamTime * PercentIn;
			const float TimeLeft		= TotalStreamTime - CurrentTime;

			// Determine if we have enough buffer to stop streaming for now
			const float MAX_BUFFERED_TIME = CVarChunkUploadDelayInSeconds.GetValueOnGameThread() * 0.5f;

			if ( TimeLeft > MAX_BUFFERED_TIME )
			{
				// Don't stream ahead by more than MAX_BUFFERED_TIME seconds
				UE_LOGF( LogHttpReplay, VeryVerbose, "FHttpNetworkReplayStreamer::ConditionallyDownloadNextChunk. Cancelling due buffer being large enough. TotalStreamTime: %2.2f, PercentIn: %2.2f, TimeLeft: %2.2f", TotalStreamTime, PercentIn, TimeLeft );
				return;
			}
		}
	}

	check( bMoreChunksDefinitelyAvailable || bStreamIsLive );

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Download the next stream chunk
	const FString URL = FString::Printf( TEXT( "%sreplay/%s/file/stream.%i" ), *ServerURL, *SessionName, StreamChunkIndex );

	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::ConditionallyDownloadNextChunk. URL: %ls", *URL );

	HttpRequest->SetURL( URL );
	HttpRequest->SetVerb( TEXT( "GET" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpDownloadFinished, StreamChunkIndex, bStreamIsLive );

	AddRequestToQueue( EQueuedHttpRequestType::DownloadingStream, HttpRequest );

	LastChunkTime = FPlatformTime::Seconds();
}

void FHttpNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	KeepReplay(ReplayName, bKeep, Delegate);
}

void FHttpNetworkReplayStreamer::KeepReplay( const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate )
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::KeepReplay. ServerURL is empty, aborting." );
		Delegate.ExecuteIfBound(FKeepReplayResult());
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Download the next stream chunk
	if ( bKeep )
	{
		HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s?shouldKeep=true" ), *ServerURL, *ReplayName ) );
	}
	else
	{
		HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s?shouldKeep=false" ), *ServerURL, *ReplayName ) );
	}

	HttpRequest->SetVerb( TEXT( "POST" ) );
	HttpRequest->SetHeader( TEXT( "Content-Type" ), TEXT( "application/json" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::KeepReplayFinished, Delegate, ReplayName );

	AddRequestToQueue( EQueuedHttpRequestType::KeepReplay, HttpRequest );
}

void FHttpNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplayFriendlyName(ReplayName, NewFriendlyName, Delegate);
}

void FHttpNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogHttpReplay, Log, "FHttpNetworkReplayStreamer::RenameReplayFriendlyName is currently unsupported.");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FHttpNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewReplayName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplay(ReplayName, NewReplayName, Delegate);
}

void FHttpNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewReplayName, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogHttpReplay, Log, "FHttpNetworkReplayStreamer::RenameReplay is currently unsupported.");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FHttpNetworkReplayStreamer::KeepReplayFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FKeepReplayCallback Delegate, FString ReplayName )
{
	RequestFinished( StreamerState, EQueuedHttpRequestType::KeepReplay, HttpRequest );

	FKeepReplayResult Result;
	if ( !bSucceeded || HttpResponse->GetResponseCode() != EHttpResponseCodes::NoContent )
	{
		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::KeepReplayFinished. FAILED, Response code: %d", HttpResponse.IsValid() ? HttpResponse->GetResponseCode() : 0 );
	}
	else
	{
		Result.Result = EStreamingOperationResult::Success;
		Result.NewReplayName = ReplayName;
	}

	Delegate.Execute(Result);
}

void FHttpNetworkReplayStreamer::RefreshViewer( const bool bFinal )
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::RefreshViewer. ServerURL is empty, aborting." );
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Download the next stream chunk
	if ( bFinal )
	{
		HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/viewer/%s?final=true" ), *ServerURL, *SessionName, *ViewerName ) );
	}
	else
	{
		HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/viewer/%s" ), *ServerURL, *SessionName, *ViewerName ) );
	}

	HttpRequest->SetVerb( TEXT( "POST" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpRefreshViewerFinished );

	AddRequestToQueue( EQueuedHttpRequestType::RefreshingViewer, HttpRequest );

	LastRefreshViewerTime = FPlatformTime::Seconds();
}

void FHttpNetworkReplayStreamer::ConditionallyRefreshViewer()
{
	if ( IsHttpRequestInFlight() || HasPendingHttpRequests() )
	{
		return;
	}

	const double REFRESH_VIEWER_IN_SECONDS = 10;

	if ( FPlatformTime::Seconds() - LastRefreshViewerTime > REFRESH_VIEWER_IN_SECONDS )
	{
		RefreshViewer( false );
	}
};

void FHttpNetworkReplayStreamer::SetLastError(FHttpReplayResult&& Result)
{
	CancelStreamingRequests();
	
	SetExtendedError(MoveTemp(Result));
}

void FHttpNetworkReplayStreamer::CancelStreamingRequests()
{
	// Cancel any in flight request
	if ( InFlightHttpRequest.IsValid() )
	{
		if ( InFlightHttpRequest->Request->OnProcessRequestComplete().IsBound() )
		{
			InFlightHttpRequest->Request->OnProcessRequestComplete().Unbind();
		}

		InFlightHttpRequest->Request->CancelRequest();
		InFlightHttpRequest = NULL;
	}

	// Empty the request queue
	QueuedHttpRequests.Empty();

	StreamerState			= EReplayStreamerState::Idle;
	bStopStreamingCalled	= false;
}

FArchive* FHttpNetworkReplayStreamer::GetHeaderArchive()
{
	return &HeaderArchive;
}

FArchive* FHttpNetworkReplayStreamer::GetStreamingArchive()
{
	return &StreamArchive;
}

FArchive* FHttpNetworkReplayStreamer::GetCheckpointArchive()
{	
	if ( SessionName.IsEmpty() )
	{
		// If we need to upload the header, we're not ready to save checkpoints
		// NOTE - The code needs to be resilient to this, and keep trying!!!!
		return NULL;
	}

	return &CheckpointArchive;
}

void FHttpNetworkReplayStreamer::UpdateTotalDemoTime( uint32 TimeInMS )
{
	TotalDemoTimeInMS = TimeInMS;
	StreamTimeRangeEnd = TimeInMS;
}

bool FHttpNetworkReplayStreamer::IsDataAvailable() const
{
	if (HasError())
	{
		return false;
	}

	if ( GotoCheckpointDelegate.IsBound() )
	{
		return false;
	}

	if ( HighPriorityEndTime > 0 )
	{
		// If we are waiting for a high priority portion of the stream, pretend like we don't have any data so that game code waits for the entire portion
		// of the high priority stream to download.
		// We do this because we assume the game wants to race through this high priority portion of the stream in a single frame
		return false;
	}

	// If we are loading, and we have more data
	if ( StreamArchive.IsLoading() && StreamArchive.Pos < StreamArchive.Buffer.Num() && NumTotalStreamChunks > 0 )
	{
		return true;
	}
	
	return false;
}

void FHttpNetworkReplayStreamer::SetHighPriorityTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS )
{
	// Set the amount of stream we should download before saying we have anymore data available
	// We will also put a high priority on this portion of the stream so that it downloads as fast as possible
	HighPriorityEndTime = EndTimeInMS;
}

bool FHttpNetworkReplayStreamer::IsDataAvailableForTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS )
{
	if (HasError())
	{
		return false;
	}

	// If the time is within the stream range we have downloaded, we will return true
	return ( StartTimeInMS >= StreamTimeRangeStart && EndTimeInMS <= StreamTimeRangeEnd );
}

bool FHttpNetworkReplayStreamer::IsLive() const 
{
	return bStreamIsLive;
}

bool FHttpNetworkReplayStreamer::IsLoadingCheckpoint() const
{
	return GotoCheckpointDelegate.IsBound();
}

void FHttpNetworkReplayStreamer::DeleteFinishedStream(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate)
{
	DeleteFinishedStream(StreamName, Delegate);
}

void FHttpNetworkReplayStreamer::DeleteFinishedStream( const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate )
{
	UE_LOGF(LogHttpReplay, Log, "FHttpNetworkReplayStreamer::DeleteFinishedStream is currently unsupported.");
	FDeleteFinishedStreamResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.ExecuteIfBound(Result);
}

void FHttpNetworkReplayStreamer::EnumerateStreams( const FNetworkReplayVersion& InReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate )
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::EnumerateStreams. ServerURL is empty, aborting." );
		Delegate.ExecuteIfBound( FEnumerateStreamsResult() );
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Build base URL
	FString URL = FString::Printf( TEXT( "%sreplay?app=%s" ), *ServerURL, *InReplayVersion.AppString );

	// Add optional stuff
	if ( InReplayVersion.Changelist != 0 )
	{
		URL += FString::Printf( TEXT( "&cl=%u" ), InReplayVersion.Changelist );
	}

	if ( InReplayVersion.NetworkVersion != 0 )
	{
		URL += FString::Printf( TEXT( "&version=%u" ), InReplayVersion.NetworkVersion );
	}

	const FString MetaStringToUse = !CVarMetaFilterOverride.GetValueOnGameThread().IsEmpty() ? CVarMetaFilterOverride.GetValueOnGameThread() : MetaString;

	// Add optional Meta parameter (filter replays by meta tag)
	if ( !MetaStringToUse.IsEmpty() )
	{
		URL += FString::Printf( TEXT( "&meta=%s" ), *MetaStringToUse );
	}

	FString UserString = GetUserStringFromUserIndex(UserIndex);

	// Add optional User parameter (filter replays by a user that was in the replay)
	if ( !UserString.IsEmpty() )
	{
		URL += FString::Printf( TEXT( "&user=%s" ), *FGenericPlatformHttp::UrlEncode( UserString ) );
	}

	// Add any extra parms now
	for ( int i = 0; i < ExtraParms.Num(); i++ )
	{
		URL += FString::Printf( TEXT( "&%s" ), *ExtraParms[i] );
	}

	HttpRequest->SetURL( URL );

	HttpRequest->SetVerb( TEXT( "GET" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished, Delegate );

	AddRequestToQueue( EQueuedHttpRequestType::EnumeratingSessions, HttpRequest );
}

void FHttpNetworkReplayStreamer::EnumerateRecentStreams( const FNetworkReplayVersion& InReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate )
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::EnumerateRecentStreams. ServerURL is empty, aborting." );
		Delegate.ExecuteIfBound( FEnumerateStreamsResult() );
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Enumerate all of the sessions
	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay?app=%s&version=%u&cl=%u&recent=%s" ), *ServerURL, *InReplayVersion.AppString, InReplayVersion.NetworkVersion, InReplayVersion.Changelist, *GetUserStringFromUserIndex(UserIndex) ) );
	HttpRequest->SetVerb( TEXT( "GET" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished, Delegate );

	AddRequestToQueue( EQueuedHttpRequestType::EnumeratingSessions, HttpRequest );
}

FQueuedHttpRequestAddUser::FQueuedHttpRequestAddUser( const FString& InUserName, TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> InHttpRequest ) : FQueuedHttpRequest( EQueuedHttpRequestType::AddingUser, InHttpRequest )
{
	FNetworkReplayUserList UserList;

	UserList.Users.Add( InUserName );
	FString JsonString = UserList.ToJson();

	Request->SetVerb( TEXT( "POST" ) );
	Request->SetContentAsString( JsonString );
	Request->SetHeader( TEXT( "Content-Type" ), TEXT( "application/json" ) );
}

bool FQueuedHttpRequestAddUser::PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName )
{
	if ( SessionName.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FQueuedHttpRequestAddUser::PreProcess. SessionName is empty." );
		return false;
	}

	//
	// Now that we have the session name, we can set the URL
	//

	Request->SetURL( FString::Printf( TEXT( "%sreplay/%s/users" ), *ServerURL, *SessionName ) );

	return true;
}

void FHttpNetworkReplayStreamer::AddUserToReplay(const FString& UserString)
{
	if ( StreamerState != EReplayStreamerState::Recording )
	{
		return;
	}

	if ( UserString.IsEmpty() )
	{
		UE_LOGF(LogHttpReplay, Log, "FHttpNetworkReplayStreamer::AddUserToReplay: can't add a user with an empty UserString.");
		return;
	}

	// Create the Http request and add to pending request list
	TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpAddUserFinished );

	// Add it as a custom event so we can snag the session name at the time of send (which we should have by then)
	AddCustomRequestToQueue( TSharedPtr< FQueuedHttpRequest >( new FQueuedHttpRequestAddUser( UserString, HttpRequest ) ) );
}

void FHttpNetworkReplayStreamer::EnumerateCheckpoints()
{
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Enumerate all of the sessions
	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/event?group=checkpoint" ), *ServerURL, *SessionName ) );
	HttpRequest->SetVerb( TEXT( "GET" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpEnumerateCheckpointsFinished );

	AddRequestToQueue( EQueuedHttpRequestType::EnumeratingCheckpoints, HttpRequest );

	LastRefreshCheckpointTime = FPlatformTime::Seconds();
}

void FHttpNetworkReplayStreamer::ConditionallyEnumerateCheckpoints()
{
	if ( IsHttpRequestInFlight() || HasPendingHttpRequests() )
	{
		return;
	}

	if ( !bStreamIsLive )
	{
		// We don't need to enumerate more than once for non live streams
		return;
	}

	// During live games, check for new checkpoints every 30 seconds
	const double REFRESH_CHECKPOINTS_IN_SECONDS = 30;

	if ( FPlatformTime::Seconds() - LastRefreshCheckpointTime > REFRESH_CHECKPOINTS_IN_SECONDS )
	{
		EnumerateCheckpoints();
	}
};

void FHttpNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents(ReplayName, Group, Delegate);
}

void FHttpNetworkReplayStreamer::EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents(SessionName, Group, Delegate);
}

void FHttpNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& EnumerationCompleteDelegate)
{
	if ( ServerURL.IsEmpty() )
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::EnumerateRecentStreams. ServerURL is empty, aborting." );
		EnumerationCompleteDelegate.ExecuteIfBound(FEnumerateEventsResult());
		return;
	}

	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();

	// Enumerate all of the events
	HttpRequest->SetURL( FString::Printf( TEXT( "%sreplay/%s/event?group=%s" ), *ServerURL, *ReplayName, *Group ) );
	HttpRequest->SetVerb( TEXT( "GET" ) );

	HttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpEnumerateEventsFinished, EnumerationCompleteDelegate );

	AddRequestToQueue( EQueuedHttpRequestType::EnumeratingCustomEvent, HttpRequest );
}

void FHttpNetworkReplayStreamer::RequestFinished(EReplayStreamerState ExpectedStreamerState, EQueuedHttpRequestType::Type ExpectedType, FHttpRequestPtr HttpRequest )
{
	check( StreamerState == ExpectedStreamerState );
	check( InFlightHttpRequest.IsValid() );
	check( InFlightHttpRequest->Request == HttpRequest );
	check( InFlightHttpRequest->Type == ExpectedType );

	InFlightHttpRequest = NULL;
};

void FHttpNetworkReplayStreamer::HttpStartUploadingFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	TSharedPtr< FQueuedHttpRequest > SavedFlightHttpRequest = InFlightHttpRequest;

	RequestFinished(EReplayStreamerState::Recording, EQueuedHttpRequestType::StartUploading, HttpRequest );

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{
		FString JsonString = HttpResponse->GetContentAsString();

		FNetworkReplayStartUploadingResponse StartUploadingResponse;

		if (!StartUploadingResponse.FromJson(JsonString))
		{
			UE_LOGF(LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpStartUploadingFinished. FromJson FAILED");
			// FIXME: Is there more I should do here?
			return;
		}

		SessionName = StartUploadingResponse.SessionId;

		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::HttpStartUploadingFinished. SessionName: %ls", *SessionName );

		// Add the session id as search tag to replay
		AddUserToReplay( SessionName );
	}
	else
	{
		if ( RetryRequest( SavedFlightHttpRequest, HttpResponse ) )
		{
			return;
		}

		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpStartUploadingFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}
}

void FHttpNetworkReplayStreamer::HttpStopUploadingFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	TSharedPtr< FQueuedHttpRequest > SavedFlightHttpRequest = InFlightHttpRequest;

	RequestFinished(EReplayStreamerState::Recording, EQueuedHttpRequestType::StopUploading, HttpRequest );

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::NoContent )
	{
		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::HttpStopUploadingFinished. SessionName: %ls", *SessionName );
	}
	else
	{
		if ( RetryRequest( SavedFlightHttpRequest, HttpResponse ) )
		{
			return;
		}

		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpStopUploadingFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}

	StreamArchive.SetIsLoading(false);
	StreamArchive.SetIsSaving(false);
	StreamArchive.Buffer.Empty();
	StreamArchive.Pos = 0;
	StreamChunkIndex = 0;
	SessionName.Empty();
}

void FHttpNetworkReplayStreamer::HttpHeaderUploadFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	TSharedPtr< FQueuedHttpRequest > SavedFlightHttpRequest = InFlightHttpRequest;

	RequestFinished(EReplayStreamerState::Recording, EQueuedHttpRequestType::UploadingHeader, HttpRequest );

	FStartStreamingResult Result;
	Result.bRecording = true;
	
	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::NoContent )
	{
		if ( HttpRequest.IsValid() )
		{
			TotalUploadBytes += HttpRequest->GetContentLength();
		}

		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpHeaderUploadFinished. TotalUploadBytes: %llu", TotalUploadBytes );
		Result.Result = EStreamingOperationResult::Success;
	}
	else
	{
		if ( RetryRequest( SavedFlightHttpRequest, HttpResponse ) )
		{
			return;
		}

		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpHeaderUploadFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}

	StartStreamingDelegate.ExecuteIfBound(Result);
	StartStreamingDelegate = FStartStreamingCallback();
}

void FHttpNetworkReplayStreamer::HttpUploadStreamFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	TSharedPtr< FQueuedHttpRequest > SavedFlightHttpRequest = InFlightHttpRequest;

	RequestFinished(EReplayStreamerState::Recording, EQueuedHttpRequestType::UploadingStream, HttpRequest );

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::NoContent )
	{
		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpUploadStreamFinished." );
	}
	else
	{
		if ( RetryRequest( SavedFlightHttpRequest, HttpResponse ) )
		{
			return;
		}

		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpUploadStreamFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}
}

void FHttpNetworkReplayStreamer::HttpUploadCheckpointFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	TSharedPtr< FQueuedHttpRequest > SavedFlightHttpRequest = InFlightHttpRequest;

	RequestFinished(EReplayStreamerState::Recording, EQueuedHttpRequestType::UploadingCheckpoint, HttpRequest );

	if ( bSucceeded && ( HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok || HttpResponse->GetResponseCode() == EHttpResponseCodes::NoContent ) )
	{
		if ( HttpRequest.IsValid() )
		{
			TotalUploadBytes += HttpRequest->GetContentLength();
		}

		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpUploadCheckpointFinished. TotalUploadBytes: %llu", TotalUploadBytes );
	}
	else
	{
		if ( RetryRequest( SavedFlightHttpRequest, HttpResponse ) )
		{
			return;
		}

		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpUploadCheckpointFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}
}

void FHttpNetworkReplayStreamer::HttpUploadCustomEventFinished(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	RequestFinished(StreamerState, EQueuedHttpRequestType::UploadingCustomEvent, HttpRequest);

	if ( bSucceeded && ( HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok || HttpResponse->GetResponseCode() == EHttpResponseCodes::NoContent ) )
	{
		if ( HttpRequest.IsValid() )
		{
			TotalUploadBytes += HttpRequest->GetContentLength();
		}

		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpUploadCustomEventFinished. TotalUploadBytes: %llu", TotalUploadBytes );
	}
	else
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpUploadCustomEventFinished. FAILED. Extra info: %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
	}
}

void FHttpNetworkReplayStreamer::HttpStartDownloadingFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::StartDownloading, HttpRequest );

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{

		FString JsonString = HttpResponse->GetContentAsString();

		FNetworkReplayStartDownloadingResponse StartDownloadingResponse;

		if (!StartDownloadingResponse.FromJson(JsonString))
		{
			UE_LOGF(LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpStartDownloadingFinished. FromJson FAILED");
			// FIXME: Is there more I should do here?
			return;
		}

		FString State = StartDownloadingResponse.State;
		ViewerName = StartDownloadingResponse.Viewer;

		bStreamIsLive = State == TEXT( "Live" );
		StreamMetadata = StartDownloadingResponse.Metadata;

		NumTotalStreamChunks = StartDownloadingResponse.NumChunks;
		TotalDemoTimeInMS = StartDownloadingResponse.Time;

		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::HttpStartDownloadingFinished. Viewer: %ls, State: %ls, NumChunks: %i, DemoTime: %2.2f", *ViewerName, *State, NumTotalStreamChunks, (float)TotalDemoTimeInMS / 1000 );

		if ( NumTotalStreamChunks == 0 )
		{			
			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpStartDownloadingFinished. NO CHUNKS" );

			StartStreamingDelegate.ExecuteIfBound(FStartStreamingResult());

			// Reset delegate
			StartStreamingDelegate = FStartStreamingCallback();

			SetLastError(EHttpReplayResult::InvalidPayload);
		}
	}
	else
	{
		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpStartDownloadingFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );

		StartStreamingDelegate.ExecuteIfBound(FStartStreamingResult());

		// Reset delegate
		StartStreamingDelegate = FStartStreamingCallback();

		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}
}

void FHttpNetworkReplayStreamer::HttpDownloadHeaderFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FDownloadHeaderCallback Delegate)
{
	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::DownloadingHeader, HttpRequest );

	check( StreamArchive.IsLoading() );

	FDownloadHeaderResult Result;
	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{
		Result.Result = EStreamingOperationResult::Success;
		HeaderArchive.Buffer.Append( HttpResponse->GetContent() );

		UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::HttpDownloadHeaderFinished. Size: %i", HeaderArchive.Buffer.Num() );
	}
	else
	{
		// FAIL
		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpDownloadHeaderFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );

		StreamArchive.Buffer.Empty();
		StartStreamingDelegate.ExecuteIfBound(FStartStreamingResult());

		// Reset delegate
		StartStreamingDelegate = FStartStreamingCallback();

		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}

	Delegate.ExecuteIfBound(Result);
}

bool FHttpNetworkReplayStreamer::DecompressResponse(FHttpResponsePtr HttpResponse, TArray<uint8>& ResultBuffer) const
{
	if ( SupportsCompression() )
	{
		SCOPE_CYCLE_COUNTER( STAT_HttpReplay_DecompressTime );

		FHttpStreamFArchive Compressed;
		Compressed.Buffer = HttpResponse->GetContent();

		TArray< uint8 > Uncompressed;
		if ( !DecompressBuffer( Compressed, Uncompressed ) )
		{
			ResultBuffer.Empty();
			UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::DecompressResponse. DecompressBuffer FAILED." );
			return false;
		}

		ResultBuffer.Append(Uncompressed);
	}
	else
	{
		ResultBuffer.Append(HttpResponse->GetContent());
	}

	return true;
}

void FHttpNetworkReplayStreamer::HttpDownloadFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 RequestedStreamChunkIndex, bool bStreamWasLive )
{
	LLM_SCOPE(ELLMTag::Replays);

	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::DownloadingStream, HttpRequest );

	check( StreamArchive.IsLoading() );

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{
		// Make sure our stream chunk index didn't change under our feet
		if ( RequestedStreamChunkIndex != StreamChunkIndex )
		{
			UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpDownloadFinished. StreamChunkIndex changed while request was in flight" );
			StreamArchive.Buffer.Empty();
			SetLastError(EHttpReplayResult::DataUnavailable);
			return;
		}

		if ( HttpResponse->GetHeader( TEXT( "NumChunks" ) ) == TEXT( "" ) )
		{
			// Assume this is an implicit status update
			UE_LOGF( LogHttpReplay, Log, "FHttpNetworkReplayStreamer::HttpDownloadFinished. NO HEADER FIELDS. Live: %i, Progress: %i / %i, Start: %i, End: %i, DemoTime: %2.2f", ( int )bStreamIsLive, StreamChunkIndex, NumTotalStreamChunks, ( int )StreamTimeRangeStart, ( int )StreamTimeRangeEnd, ( float )TotalDemoTimeInMS / 1000 );
			return;
		}

		const int32 NewNumTotalStreamChunks = FCString::Atoi( *HttpResponse->GetHeader( TEXT( "NumChunks" ) ) );
		const uint32 NewTotalDemoTimeInMS = FCString::Atoi( *HttpResponse->GetHeader( TEXT( "Time" ) ) );

		// Cached http calls can make time go backwards, so protect against that
		if ( NewNumTotalStreamChunks > NumTotalStreamChunks )
		{
			NumTotalStreamChunks = NewNumTotalStreamChunks;
		}

		if ( NewTotalDemoTimeInMS > TotalDemoTimeInMS )
		{
			TotalDemoTimeInMS = NewTotalDemoTimeInMS;
		}

		const bool bNewStreamIsLive = HttpResponse->GetHeader( TEXT( "State" ) ) == TEXT( "Live" );

		if ( bStreamIsLive && !bNewStreamIsLive )
		{
			bStreamIsLive = bNewStreamIsLive;
		}

		if ( HttpResponse->GetContent().Num() > 0 || bStreamIsLive )
		{
			if ( HttpResponse->GetContent().Num() > 0 )
			{
				if ( StreamArchive.Buffer.Num() == 0 )
				{
					// If we haven't started streaming yet, this will be the start of this stream
					StreamTimeRangeStart = FCString::Atoi( *HttpResponse->GetHeader( TEXT( "MTime1" ) ) );
				}

				// This is the new end of the stream
				StreamTimeRangeEnd = FCString::Atoi( *HttpResponse->GetHeader( TEXT( "MTime2" ) ) );

				// Add response to response cache
				AddResponseToCache( HttpRequest, HttpResponse );
				
				if (!DecompressResponse(HttpResponse, StreamArchive.Buffer))
				{
					SetLastError(EHttpReplayResult::DecompressionFailed);
					return;
				}

				StreamChunkIndex++;
			}
			else if ( HighPriorityEndTime != 0 )
			{
				// We failed to download live content during fast forward
				// This can happen in the rare case that a checkpoint told us to download a file that is being held on to, due to delay mechanism on backend
				UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpDownloadFinished. HighPriorityEndTime != 0. Live: %i, Progress: %i / %i, Start: %i, End: %i, DemoTime: %2.2f", ( int )bStreamIsLive, StreamChunkIndex, NumTotalStreamChunks, ( int )StreamTimeRangeStart, ( int )StreamTimeRangeEnd, ( float )TotalDemoTimeInMS / 1000 );
				HighPriorityEndTime = 0;
			}

			UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpDownloadFinished. Live: %i, Progress: %i / %i, Start: %i, End: %i, DemoTime: %2.2f", (int)bStreamIsLive, StreamChunkIndex, NumTotalStreamChunks, (int)StreamTimeRangeStart, (int)StreamTimeRangeEnd, (float)TotalDemoTimeInMS / 1000 );
		}
		else
		{
			// If we're not live, and we weren't live before, this shouldn't fail
			// The reason we allow previously live requests fail, is that we might have requested the next file in the stream, but if the game server crashes, the replay will flip to "not live", and won't have that file
			if ( !bStreamWasLive )
			{
				UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpDownloadFinished. FAILED (no content when not live). Live: %i, Progress: %i / %i, Start: %i, End: %i, DemoTime: %2.2f. %ls", ( int )bStreamIsLive, StreamChunkIndex, NumTotalStreamChunks, ( int )StreamTimeRangeStart, ( int )StreamTimeRangeEnd, ( float )TotalDemoTimeInMS / 1000, *BuildRequestErrorString( HttpRequest, HttpResponse ) );
				StreamArchive.Buffer.Empty();
				SetLastError(EHttpReplayResult::InvalidHttpResponse);
			}
		}		
	}
	else
	{
		if ( bStreamIsLive )
		{
			bStreamIsLive = false;

			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpDownloadFinished. Failed live, turning off live flag. Response code: %d, Live: %i, Progress: %i / %i, Start: %i, End: %i, DemoTime: %2.2f", HttpResponse.IsValid() ? HttpResponse->GetResponseCode() : 0, ( int )bStreamIsLive, StreamChunkIndex, NumTotalStreamChunks, ( int )StreamTimeRangeStart, ( int )StreamTimeRangeEnd, ( float )TotalDemoTimeInMS / 1000 );
		}
		else
		{
			UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpDownloadFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
			StreamArchive.Buffer.Empty();
			SetLastError(EHttpReplayResult::InvalidHttpResponse);
		}
	}
}

void FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished." );

	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::DownloadingCheckpoint, HttpRequest );

	check( StreamArchive.IsLoading() );
	check( GotoCheckpointDelegate.IsBound() );
	check( DownloadCheckpointIndex >= 0 );

	// If we get here after StopStreaming was called, then assume this operation should be cancelled
	// A more correct fix would be to actually cancel this in-flight request when StopStreaming is called
	// But for now, this is a safe change, and can co-exist with the more proper fix
	if ( bStopStreamingCalled )
	{
		GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
		GotoCheckpointDelegate	= FGotoCallback();
		DownloadCheckpointIndex = -1;
		LastGotoTimeInMS		= -1;
		return;
	}

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{
		if ( HttpResponse->GetContent().Num() == 0 )
		{
			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished. Checkpoint empty." );
			GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
			GotoCheckpointDelegate = FGotoCallback();
			return;
		}

		// Add response to response cache
		AddResponseToCache( HttpRequest, HttpResponse );

		CheckpointArchive.Buffer.Empty();

		// Get the checkpoint data
		if (!DecompressResponse(HttpResponse, CheckpointArchive.Buffer))
		{
			GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
			GotoCheckpointDelegate	= FGotoCallback();
			DownloadCheckpointIndex = -1;
			LastGotoTimeInMS		= -1;
			return;
		}

		CheckpointArchive.Pos = 0;

		// Completely reset our stream (we're going to start downloading from the start of the checkpoint)
		StreamArchive.Buffer.Empty();
		StreamArchive.Pos				= 0;
		StreamArchive.bAtEndOfReplay	= false;

		// Reset any time we were waiting on in the past
		HighPriorityEndTime	= 0;

		// Reset our stream range
		StreamTimeRangeStart	= 0;
		StreamTimeRangeEnd		= 0;

		// Set the next chunk to be right after this checkpoint (which was stored in the metadata)
		const FReplayEventListItem& Checkpoint = CheckpointList.ReplayEvents[DownloadCheckpointIndex];
		StreamChunkIndex = FCString::Atoi( *Checkpoint.Metadata );

		LastChunkTime = 0;		// Force the next chunk to start downloading immediately

		// If we want to fast forward past the end of a stream, clamp to the checkpoint
		if ( LastGotoTimeInMS >= 0 && StreamChunkIndex >= NumTotalStreamChunks )
		{
			UE_LOG( LogHttpReplay, Warning, TEXT( "FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished. Clamped to checkpoint: %" INT64_FMT ), LastGotoTimeInMS );

			StreamTimeRangeStart	= Checkpoint.Time1;
			StreamTimeRangeEnd		= Checkpoint.Time1;
			LastGotoTimeInMS		= -1;
		}

		if ( LastGotoTimeInMS >= 0 )
		{
			// If we are fine scrubbing, make sure to wait on the part of the stream that is needed to do this in one frame
			SetHighPriorityTimeRange(Checkpoint.Time1, IntCastChecked<uint32>(LastGotoTimeInMS));
			
			// Subtract off checkpoint time so we pass in the leftover to the engine to fast forward through for the fine scrubbing part
			LastGotoTimeInMS -= Checkpoint.Time1;
		}

		// Notify game code of success
		FGotoResult Result;
		Result.Result = EStreamingOperationResult::Success;
		Result.ExtraTimeMS = LastGotoTimeInMS;
		Result.CheckpointInfo.CheckpointIndex = DownloadCheckpointIndex;
		Result.CheckpointInfo.CheckpointStartTime = Checkpoint.Time1;

		GotoCheckpointDelegate.ExecuteIfBound(Result);

		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished. SUCCESS. StreamChunkIndex: %i", StreamChunkIndex );
	}
	else
	{
		// Oops, something went wrong, notify game code of failure
		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
	}

	// Reset things
	GotoCheckpointDelegate	= FGotoCallback();
	DownloadCheckpointIndex = -1;
	LastGotoTimeInMS		= -1;
}

void FHttpNetworkReplayStreamer::HttpRefreshViewerFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	TSharedPtr< FQueuedHttpRequest > SavedFlightHttpRequest = InFlightHttpRequest;

	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::RefreshingViewer, HttpRequest );

	if ( !bSucceeded || HttpResponse->GetResponseCode() != EHttpResponseCodes::NoContent )
	{
		static const int32 MaxViewerRefreshFails = 6;

		if ( ++RefreshViewerFails > MaxViewerRefreshFails )
		{
			UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpRefreshViewerFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
			SetLastError(EHttpReplayResult::InvalidHttpResponse);
		}
		else
		{
			const int32 RetriesLeft = MaxViewerRefreshFails - RefreshViewerFails;
			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpRefreshViewerFinished. Failed. Retries left: %i, %ls", RetriesLeft, *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		}
	}
	else
	{
		RefreshViewerFails = 0;
	}
}

void FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FEnumerateStreamsCallback Delegate )
{
	check( InFlightHttpRequest.IsValid() );
	check( InFlightHttpRequest->Request == HttpRequest );
	check( InFlightHttpRequest->Type == EQueuedHttpRequestType::EnumeratingSessions );

	InFlightHttpRequest = NULL;

	FEnumerateStreamsResult Result;
	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{		
		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished." );

		TArray<FNetworkReplayStreamInfo> Streams;
		FString JsonString = HttpResponse->GetContentAsString();

		FNetworkReplayList ReplayList;

		if ( !ReplayList.FromJson( JsonString ) )
		{
			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished. FromJson FAILED" );
		}
		else
		{
			Result.Result = EStreamingOperationResult::Success;
			for (int32 i = 0; i < ReplayList.Replays.Num(); i++)
			{
				FNetworkReplayStreamInfo NewStream;

				NewStream.Name = ReplayList.Replays[i].SessionName;
				NewStream.FriendlyName = ReplayList.Replays[i].FriendlyName;
				NewStream.Timestamp = ReplayList.Replays[i].Timestamp;
				NewStream.SizeInBytes = ReplayList.Replays[i].SizeInBytes;
				NewStream.LengthInMS = ReplayList.Replays[i].DemoTimeInMs;
				NewStream.NumViewers = ReplayList.Replays[i].NumViewers;
				NewStream.bIsLive = ReplayList.Replays[i].bIsLive;
				NewStream.Changelist = ReplayList.Replays[i].Changelist;
				NewStream.bShouldKeep = ReplayList.Replays[i].bShouldKeep;

				Result.FoundStreams.Add(NewStream);
			}
		}
	}
	else
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpEnumerateSessionsFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
	}

	Delegate.Execute(Result);
}

void FHttpNetworkReplayStreamer::HttpEnumerateCheckpointsFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::EnumeratingCheckpoints, HttpRequest );

	FStartStreamingResult Result;
	Result.bRecording = false;

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{		
		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpEnumerateCheckpointsFinished." );

		FString JsonString = HttpResponse->GetContentAsString();

		CheckpointList.ReplayEvents.Empty();

		if ( !CheckpointList.FromJson( JsonString ) )
		{
			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpEnumerateCheckpointsFinished. FromJson FAILED" );
			SetLastError(EHttpReplayResult::FailedJsonParse);
		}
		else
		{
			// Sort checkpoints by time
			Algo::SortBy(CheckpointList.ReplayEvents, &FReplayEventListItem::Time1);

			Result.Result = EStreamingOperationResult::Success;
		}	
	}
	else
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpEnumerateCheckpointsFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}
	
	StartStreamingDelegate.ExecuteIfBound(Result);
	StartStreamingDelegate = FStartStreamingCallback();
}

void FHttpNetworkReplayStreamer::HttpEnumerateEventsFinished(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FEnumerateEventsCallback Delegate)
{
	RequestFinished(StreamerState, EQueuedHttpRequestType::EnumeratingCustomEvent, HttpRequest);

	FEnumerateEventsResult Result;

	if (bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok)
	{
		FReplayEventList EventList;
		FString JsonString = HttpResponse->GetContentAsString();
		if (Result.ReplayEventList.FromJson(JsonString))
		{
			UE_LOGF(LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpEnumerateEventsFinished. %ls", *JsonString);
			Result.Result = EStreamingOperationResult::Success;
		}
		else
		{
			UE_LOGF(LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpEnumerateEventsFinished. FromJson FAILED");
			SetLastError(EHttpReplayResult::FailedJsonParse);
		}
	}
	else
	{
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpEnumerateEventsFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		SetLastError(EHttpReplayResult::InvalidHttpResponse);
	}
	
	Delegate.ExecuteIfBound(Result);
}

void FHttpNetworkReplayStreamer::HttpAddUserFinished(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	RequestFinished(EReplayStreamerState::Recording, EQueuedHttpRequestType::AddingUser, HttpRequest );

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::NoContent )
	{		
		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpAddUserFinished." );
	}
	else
	{
		// Don't consider this a fatal error
		UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpAddUserFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
	}
}

bool FHttpNetworkReplayStreamer::ProcessNextHttpRequest()
{
	if ( IsHttpRequestInFlight() )
	{
		if ( InFlightHttpRequest->NextRetryTime > 0.0f )
		{
			if ( FPlatformTime::Seconds() > InFlightHttpRequest->NextRetryTime )
			{
				InFlightHttpRequest->NextRetryTime = 0.0f;
				InFlightHttpRequest->Request->ProcessRequest();
			}
		}

		// We only process one http request at a time to keep things simple
		return false;
	}

	if ( QueuedHttpRequests.Num() > 0 )
	{
		TSharedPtr< FQueuedHttpRequest > QueuedRequest = QueuedHttpRequests[0];

		QueuedHttpRequests.RemoveAt( 0 );

		UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::ProcessNextHttpRequest. Dequeue Type: %ls", EQueuedHttpRequestType::ToString( QueuedRequest->Type ) );

		check( !InFlightHttpRequest.IsValid() );

		// Check for a couple special requests that aren't really http calls, but just using the request system for order of operations management
		if ( QueuedRequest->Type == EQueuedHttpRequestType::UploadHeader )
		{
			check( !SessionName.IsEmpty() );
			UploadHeader();
			return ProcessNextHttpRequest();
		}
		else if ( QueuedRequest->Type == EQueuedHttpRequestType::StopStreaming )
		{
			check( IsStreaming() );
			StreamerState = EReplayStreamerState::Idle;
			bStopStreamingCalled = false;
			ensure( QueuedHttpRequests.Num() == 0 );
			return ProcessNextHttpRequest();
		}

		if ( !QueuedRequest->PreProcess( this, ServerURL, SessionName ) )
		{
			// This request failed, go ahead and process the next request
			return ProcessNextHttpRequest();
		}

		// If this task has no http request, immediately go to the next task
		if ( !QueuedRequest->Request.IsValid() )
		{
			return ProcessNextHttpRequest();
		}

		InFlightHttpRequest = QueuedRequest;

		FCachedResponse* CachedResponse = ResponseCache.Find( InFlightHttpRequest->Request->GetURL() );

		if ( CachedResponse != nullptr )
		{
			if ( InFlightHttpRequest->Request->OnProcessRequestComplete().IsBound() )
			{
				// If we have this response in the cache, process it now
				CachedResponse->LastAccessTime = FPlatformTime::Seconds();
				InFlightHttpRequest->Request->OnProcessRequestComplete().ExecuteIfBound( InFlightHttpRequest->Request, CachedResponse->Response, true );

				// We can now immediately process next request
				return ProcessNextHttpRequest();
			}
			else
			{
				// Should never happen, but just in case
				ResponseCache.Remove( InFlightHttpRequest->Request->GetURL() );
			}
		}

		ProcessRequestInternal( InFlightHttpRequest->Request );

		// We can return now since we know a http request is now in flight
		return true;
	}

	return false;
}

void FHttpNetworkReplayStreamer::ProcessRequestInternal( TSharedPtr< class IHttpRequest, ESPMode::ThreadSafe > Request )
{
	Request->ProcessRequest();
}

void FHttpNetworkReplayStreamer::Tick( const float DeltaTime )
{
	LLM_SCOPE(ELLMTag::Replays);

	// Attempt to process the next http request
	if ( ProcessNextHttpRequest() )
	{
		check( IsHttpRequestInFlight() );
	}

	if ( bStopStreamingCalled )
	{
		return;
	}

	if ( StreamerState == EReplayStreamerState::Recording )
	{
		ConditionallyFlushStream();
	}
	else if ( StreamerState == EReplayStreamerState::Playback )
	{
		if ( IsTaskPendingOrInFlight( EQueuedHttpRequestType::StartDownloading ) )
		{
			// If we're still waiting on finalizing the download request then return
			return;
		}

		// Check to see if we're done downloading the high priority portion of the strema
		// If so, we can cancel the request
		if ( HighPriorityEndTime > 0 && StreamTimeRangeEnd >= HighPriorityEndTime )
		{
			HighPriorityEndTime = 0;
		}

		// Check to see if we're at the end of non live streams
		if ( StreamChunkIndex >= NumTotalStreamChunks && !bStreamIsLive )
		{
			// Make note of when we reach the end of non live stream
			StreamArchive.bAtEndOfReplay = true;
		}

		ConditionallyRefreshViewer();
		ConditionallyDownloadNextChunk();
		ConditionallyEnumerateCheckpoints();
	}
}

bool FHttpNetworkReplayStreamer::IsHttpRequestInFlight() const
{
	return InFlightHttpRequest.IsValid();
}

bool FHttpNetworkReplayStreamer::HasPendingHttpRequests() const
{
	// If there is currently one in flight, or we have more to process, return true
	return IsHttpRequestInFlight() || QueuedHttpRequests.Num() > 0;
}

bool FHttpNetworkReplayStreamer::IsStreaming() const
{
	return StreamerState != EReplayStreamerState::Idle;
}

void FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded )
{
	UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished." );

	RequestFinished(EReplayStreamerState::Playback, EQueuedHttpRequestType::DownloadingCheckpoint, HttpRequest );

	check( StreamArchive.IsLoading() );
	check( GotoCheckpointDelegate.IsBound() );
	check( DownloadCheckpointIndex >= 0 );
	check( DeltaDownloadCheckpointIndex >= 0);

	// If we get here after StopStreaming was called, then assume this operation should be cancelled
	// A more correct fix would be to actually cancel this in-flight request when StopStreaming is called
	// But for now, this is a safe change, and can co-exist with the more proper fix
	if ( bStopStreamingCalled )
	{
		GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
		GotoCheckpointDelegate	= FGotoCallback();
		DownloadCheckpointIndex = -1;
		DeltaDownloadCheckpointIndex = -1;
		LastGotoTimeInMS		= -1;
		return;
	}

	if ( bSucceeded && HttpResponse->GetResponseCode() == EHttpResponseCodes::Ok )
	{
		if ( HttpResponse->GetContent().Num() == 0 )
		{
			UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished. Checkpoint empty." );
			GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
			GotoCheckpointDelegate = FGotoCallback();
			DownloadCheckpointIndex = -1;
			DeltaDownloadCheckpointIndex = -1;
			LastGotoTimeInMS		= -1;
			return;
		}

		// Add response to response cache
		AddResponseToCache( HttpRequest, HttpResponse );

		// clear archive when receiving first chunk, then append
		if (DeltaDownloadCheckpointIndex == 0)
		{
			CheckpointArchive.Buffer.Empty();
		}

		uint32 CheckpointSize = 0;

		FMemoryReader ContentAr(HttpResponse->GetContent(), true);
		ContentAr << CheckpointSize;

		FMemoryWriter CheckpointAr(CheckpointArchive.Buffer, true);
		CheckpointAr.Seek(CheckpointAr.TotalSize());
		CheckpointAr << CheckpointSize;

		// Get the checkpoint data
		if (!DecompressResponse(HttpResponse, CheckpointArchive.Buffer))
		{
			GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
			GotoCheckpointDelegate	= FGotoCallback();
			DownloadCheckpointIndex = -1;
			DeltaDownloadCheckpointIndex = -1;
			LastGotoTimeInMS		= -1;
			return;
		}

		if (DeltaDownloadCheckpointIndex < DownloadCheckpointIndex)
		{
			DeltaDownloadCheckpointIndex++;
			check(CheckpointList.ReplayEvents.IsValidIndex(DeltaDownloadCheckpointIndex));

			TSharedRef<IHttpRequest, ESPMode::ThreadSafe> NextHttpRequest = FHttpModule::Get().CreateRequest();

			// Download the next stream chunk
			NextHttpRequest->SetURL( FString::Printf( TEXT( "%sevent/%s" ), *ServerURL,  *CheckpointList.ReplayEvents[DeltaDownloadCheckpointIndex].ID ) );
			NextHttpRequest->SetVerb( TEXT( "GET" ) );

			NextHttpRequest->OnProcessRequestComplete().BindRaw( this, &FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished );

			AddRequestToQueue( EQueuedHttpRequestType::DownloadingCheckpoint, NextHttpRequest );
			return;
		}
		else
		{
			CheckpointArchive.Pos = 0;

			// Completely reset our stream (we're going to start downloading from the start of the checkpoint)
			StreamArchive.Buffer.Empty();
			StreamArchive.Pos				= 0;
			StreamArchive.bAtEndOfReplay	= false;

			// Reset any time we were waiting on in the past
			HighPriorityEndTime	= 0;

			// Reset our stream range
			StreamTimeRangeStart	= 0;
			StreamTimeRangeEnd		= 0;

			// Set the next chunk to be right after this checkpoint (which was stored in the metadata)
			StreamChunkIndex = FCString::Atoi( *CheckpointList.ReplayEvents[ DownloadCheckpointIndex ].Metadata );

			LastChunkTime = 0;		// Force the next chunk to start downloading immediately

			// If we want to fast forward past the end of a stream, clamp to the checkpoint
			if ( LastGotoTimeInMS >= 0 && StreamChunkIndex >= NumTotalStreamChunks )
			{
				UE_LOG( LogHttpReplay, Warning, TEXT( "FHttpNetworkReplayStreamer::HttpDownloadCheckpointFinished. Clamped to checkpoint: %" INT64_FMT ), LastGotoTimeInMS );

				StreamTimeRangeStart	= CheckpointList.ReplayEvents[DownloadCheckpointIndex].Time1;
				StreamTimeRangeEnd		= CheckpointList.ReplayEvents[DownloadCheckpointIndex].Time1;
				LastGotoTimeInMS		= -1;
			}

			if ( LastGotoTimeInMS >= 0 )
			{
				// If we are fine scrubbing, make sure to wait on the part of the stream that is needed to do this in one frame
				SetHighPriorityTimeRange(CheckpointList.ReplayEvents[ DownloadCheckpointIndex ].Time1, IntCastChecked<uint32>(LastGotoTimeInMS));

				// Subtract off checkpoint time so we pass in the leftover to the engine to fast forward through for the fine scrubbing part
				LastGotoTimeInMS -= CheckpointList.ReplayEvents[ DownloadCheckpointIndex ].Time1;
			}

			// Notify game code of success
			FGotoResult Result;
			Result.Result = EStreamingOperationResult::Success;
			Result.ExtraTimeMS = LastGotoTimeInMS;
			GotoCheckpointDelegate.ExecuteIfBound(Result);

			UE_LOGF( LogHttpReplay, Verbose, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished. SUCCESS. StreamChunkIndex: %i", StreamChunkIndex );
		}
	}
	else
	{
		// Oops, something went wrong, notify game code of failure
		UE_LOGF( LogHttpReplay, Error, "FHttpNetworkReplayStreamer::HttpDownloadCheckpointDeltaFinished. FAILED, %ls", *BuildRequestErrorString( HttpRequest, HttpResponse ) );
		GotoCheckpointDelegate.ExecuteIfBound(FGotoResult());
	}

	// Reset things
	GotoCheckpointDelegate	= FGotoCallback();
	DownloadCheckpointIndex = -1;
	DeltaDownloadCheckpointIndex = -1;
	LastGotoTimeInMS		= -1;
}

bool FHttpNetworkReplayStreamer::IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const
{
	bool bSupported = false;

	switch (CheckpointType)
	{
	case EReplayCheckpointType::Full:
	case EReplayCheckpointType::Delta:
		bSupported = true;
		break;
	}

	return bSupported;
}

IMPLEMENT_MODULE( FHttpNetworkReplayStreamingFactory, HttpNetworkReplayStreaming )

TSharedPtr< INetworkReplayStreamer > FHttpNetworkReplayStreamingFactory::CreateReplayStreamer()
{
	TSharedPtr< FHttpNetworkReplayStreamer > Streamer( new FHttpNetworkReplayStreamer );

	HttpStreamers.Add( Streamer );

	return Streamer;
}

void FHttpNetworkReplayStreamingFactory::Tick( float DeltaTime )
{
	for ( int i = HttpStreamers.Num() - 1; i >= 0; i-- )
	{
		check( HttpStreamers[i].IsValid() );

		HttpStreamers[i]->Tick( DeltaTime );
		
		// We can release our hold when streaming is completely done
		if ( HttpStreamers[i].IsUnique() && !HttpStreamers[i]->HasPendingHttpRequests() )
		{
			if ( HttpStreamers[i]->IsStreaming() )
			{
				UE_LOGF( LogHttpReplay, Warning, "FHttpNetworkReplayStreamingFactory::Tick. Stream was stopped early." );
			}

			HttpStreamers.RemoveAt( i );
		}
	}
}

TStatId FHttpNetworkReplayStreamingFactory::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT( FHttpNetworkReplayStreamingFactory, STATGROUP_Tickables );
}

bool FHttpNetworkReplayStreamingFactory::HasPendingHttpRequests() const
{
	bool bPendingRequests = false;

	for (const TSharedPtr<FHttpNetworkReplayStreamer>& Streamer : HttpStreamers)
	{
		if (Streamer.IsValid() && Streamer->HasPendingHttpRequests())
		{
			bPendingRequests = true;
			break;
		}
	}

	return bPendingRequests;
}

void FHttpNetworkReplayStreamingFactory::Flush()
{
	// defaulting to false as that was the previous behavior
	bool bFlushStreamersOnShutdown = false;
	GConfig->GetBool(TEXT("HttpNetworkReplayStreamingFactory"), TEXT("bFlushStreamersOnShutdown"), bFlushStreamersOnShutdown, GEngineIni);

	if (bFlushStreamersOnShutdown)
	{
		double MaxFlushTimeSeconds = -1.0;
		GConfig->GetDouble(TEXT("HttpNetworkReplayStreamingFactory"), TEXT("MaxFlushTimeSeconds"), MaxFlushTimeSeconds, GEngineIni);

		const double BeginWaitTime = FPlatformTime::Seconds();
		double LastTime = BeginWaitTime;
		while (HasPendingHttpRequests())
		{
			const double AppTime = FPlatformTime::Seconds();
			const double TotalWait = AppTime - BeginWaitTime;

			if ((MaxFlushTimeSeconds > 0) && (TotalWait > MaxFlushTimeSeconds))
			{
				UE_LOGF(LogHttpReplay, Display, "Abandoning streamer flush after waiting %0.2f seconds", TotalWait);
				break;
			}

			const float DeltaTime = FloatCastChecked<float>(AppTime - LastTime, UE::LWC::DefaultFloatPrecision);
			Tick(DeltaTime);

			LastTime = AppTime;

			if (HasPendingHttpRequests())
			{
				const float SleepInterval = 0.05f;

				FHttpModule::Get().GetHttpManager().Flush(EHttpFlushReason::Default);

				FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

				if (FPlatformProcess::SupportsMultithreading() || FForkProcessHelper::SupportsMultithreadingPostFork())
				{
					UE_LOGF(LogHttpReplay, Display, "Sleeping to wait for outstanding requests.");
					FPlatformProcess::Sleep(SleepInterval);
				}
			}
		}
	}
}

void FHttpNetworkReplayStreamingFactory::ShutdownModule()
{
	Flush();
}

#define CASE_EHTTPREPLAYRESULT_TO_TEXT_RET(txt) case txt: ReturnVal = TEXT(#txt); break;

const TCHAR* LexToString(EHttpReplayResult Enum)
{
	const TCHAR* ReturnVal = TEXT("::Invalid");

	switch (Enum)
	{
		FOREACH_ENUM_EHTTPREPLAYRESULT(CASE_EHTTPREPLAYRESULT_TO_TEXT_RET)
	}

	while (*ReturnVal != ':')
	{
		ReturnVal++;
	}

	ReturnVal += 2;

	return ReturnVal;
}

#undef CASE_EHTTPREPLAYRESULT_TO_TEXT_RET
