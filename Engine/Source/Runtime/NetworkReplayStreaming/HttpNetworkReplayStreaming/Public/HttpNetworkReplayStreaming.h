// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "NetworkReplayStreaming.h"
#include "Interfaces/IHttpRequest.h"
#include "Tickable.h"
#include "HttpNetworkReplayStreaming.generated.h"

#define UE_API HTTPNETWORKREPLAYSTREAMING_API

class FHttpNetworkReplayStreamer;

/**
 * Archive used to buffer stream over http
 */
class FHttpStreamFArchive : public FArchive
{
public:
	FHttpStreamFArchive() : Pos( 0 ), bAtEndOfReplay( false ) {}

	UE_API virtual void	Serialize( void* V, int64 Length );
	UE_API virtual int64	Tell();
	UE_API virtual int64	TotalSize();
	UE_API virtual void	Seek( int64 InPos );
	UE_API virtual bool	AtEnd();

	TArray< uint8 >	Buffer;
	int32			Pos;
	bool			bAtEndOfReplay;
};

namespace EQueuedHttpRequestType
{
	enum Type
	{
		StartUploading,				// We have made a request to start uploading a replay
		UploadingHeader,			// We are uploading the replay header
		UploadingStream,			// We are in the process of uploading the replay stream
		StopUploading,				// We have made the request to stop uploading a live replay stream
		StartDownloading,			// We have made the request to start downloading a replay stream
		DownloadingHeader,			// We are downloading the replay header
		DownloadingStream,			// We are in the process of downloading the replay stream
		RefreshingViewer,			// We are refreshing the server to let it know we're still viewing
		EnumeratingSessions,		// We are in the process of downloading the available sessions
		EnumeratingCheckpoints,		// We are in the process of downloading the available checkpoints
		UploadingCheckpoint,		// We are uploading a checkpoint
		DownloadingCheckpoint,		// We are downloading a checkpoint
		AddingUser,					// We are adding a user who joined in progress during recording
		UploadingCustomEvent,		// We are uploading a custom event
		EnumeratingCustomEvent,		// We are in the process of enumerating a custom event set
		RequestEventData,			// We are in the process of requesting the data for a specific event
		UploadHeader,				// Request to upload header (has to be done after we get info from server)
		StopStreaming,				// Request to stop streaming
		KeepReplay,					// Request to keep replay (or cancel keeping replay)
	};

	inline const TCHAR* ToString( EQueuedHttpRequestType::Type Type )
	{
		switch ( Type )
		{
			case StartUploading:
				return TEXT( "StartUploading" );
			case UploadingHeader:
				return TEXT( "UploadingHeader" );
			case UploadingStream:
				return TEXT( "UploadingStream" );
			case StopUploading:
				return TEXT( "StopUploading" );
			case StartDownloading:
				return TEXT( "StartDownloading" );
			case DownloadingHeader:
				return TEXT( "DownloadingHeader" );
			case DownloadingStream:
				return TEXT( "DownloadingStream" );
			case RefreshingViewer:
				return TEXT( "RefreshingViewer" );
			case EnumeratingSessions:
				return TEXT( "EnumeratingSessions" );
			case EnumeratingCheckpoints:
				return TEXT( "EnumeratingCheckpoints" );
			case UploadingCheckpoint:
				return TEXT( "UploadingCheckpoint" );
			case DownloadingCheckpoint:
				return TEXT( "DownloadingCheckpoint" );
			case AddingUser:
				return TEXT( "AddingUser" );
			case UploadingCustomEvent:
				return TEXT( "UploadingCustomEvent" );
			case EnumeratingCustomEvent:
				return TEXT( "EnumeratingCustomEvent" );
			case RequestEventData:
				return TEXT("RequestEventData");
			case UploadHeader:
				return TEXT( "UploadHeader" );
			case StopStreaming:
				return TEXT( "StopStreaming" );
			case KeepReplay:
				return TEXT( "KeepReplay" );
		}

		return TEXT( "Unknown EQueuedHttpRequestType type." );
	}
};

class FHttpNetworkReplayStreamer;

class FQueuedHttpRequest
{
public:
	FQueuedHttpRequest( const EQueuedHttpRequestType::Type InType, TSharedPtr< class IHttpRequest, ESPMode::ThreadSafe > InRequest ) : Type( InType ), Request( InRequest ), RetryProgress( 0 ), MaxRetries( 0 ), RetryDelay( 0.0f ), NextRetryTime( 0.0 )
	{
	}

	FQueuedHttpRequest( const EQueuedHttpRequestType::Type InType, TSharedPtr< class IHttpRequest, ESPMode::ThreadSafe > InRequest, const int32 InMaxRetries, const float InRetryDelay ) : Type( InType ), Request( InRequest ), RetryProgress( 0 ), MaxRetries( InMaxRetries ), RetryDelay( InRetryDelay ), NextRetryTime( 0.0 )
	{
	}

	virtual ~FQueuedHttpRequest() = default;

	EQueuedHttpRequestType::Type		Type;
	TSharedPtr< class IHttpRequest, ESPMode::ThreadSafe >	Request;

	int32								RetryProgress;
	int32								MaxRetries;
	float								RetryDelay;
	double								NextRetryTime;

	virtual bool PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName )
	{
		return true;
	}
};

/**
* FQueuedHttpRequestAddEvent
* Custom event so that we can defer the need to knowing SessionName until we actually send it (which we should have it by then, since requests are executed in order)
*/
class FQueuedHttpRequestAddEvent : public FQueuedHttpRequest
{
public:
	FQueuedHttpRequestAddEvent( const FString& InName, const uint32 InTimeInMS, const FString& InGroup, const FString& InMeta, const TArray<uint8>& InData, TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> InHttpRequest );

	virtual ~FQueuedHttpRequestAddEvent() = default;

	virtual bool PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName ) override;

	FString		Name;
	uint32		TimeInMS;
	FString		Group;
	FString		Meta;
};

/**
* FQueuedHttpRequestAddUser
* Custom event so that we can defer the need to knowing SessionName until we actually send it (which we should have it by then, since requests are executed in order)
*/
class FQueuedHttpRequestAddUser : public FQueuedHttpRequest
{
public:
	FQueuedHttpRequestAddUser( const FString& InUser, TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> InHttpRequest );

	virtual ~FQueuedHttpRequestAddUser() = default;

	virtual bool PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName ) override;
};

/**
* FQueuedGotoFakeCheckpoint
*/
class FQueuedGotoFakeCheckpoint : public FQueuedHttpRequest
{
public:
	FQueuedGotoFakeCheckpoint();

	virtual ~FQueuedGotoFakeCheckpoint() = default;

	virtual bool PreProcess( FHttpNetworkReplayStreamer* Streamer, const FString& ServerURL, const FString& SessionName ) override;
};

class FCachedResponse
{
public:
	FCachedResponse( FHttpResponsePtr InResponse, const double InLastAccessTime ) : Response( InResponse ), LastAccessTime( InLastAccessTime )
	{
	}
	
	FHttpResponsePtr	Response;
	double				LastAccessTime;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UENUM()
enum class EHttpReplayResult : uint32
{
	Success,
	FailedJsonParse,
	DataUnavailable,
	InvalidHttpResponse,
	CompressionFailed,
	DecompressionFailed,
	InvalidPayload,
	Unknown,
};

DECLARE_NETRESULT_ENUM(EHttpReplayResult);

HTTPNETWORKREPLAYSTREAMING_API const TCHAR* LexToString(EHttpReplayResult Enum);

/**
 * Http network replay streaming manager
 */
class FHttpNetworkReplayStreamer : public INetworkReplayStreamer
{
	using FHttpReplayResult = UE::Net::TNetResult<EHttpReplayResult>;

public:
	FHttpNetworkReplayStreamer();

	/** INetworkReplayStreamer implementation */
	virtual void		StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate) override;
	virtual void		StopStreaming() override;
	virtual FArchive*	GetHeaderArchive() override;
	virtual FArchive*	GetStreamingArchive() override;
	virtual FArchive*	GetCheckpointArchive() override;
	virtual void		FlushCheckpoint( const uint32 TimeInMS ) override;
	virtual void		GotoCheckpointIndex( const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType ) override;
	virtual void		GotoTimeInMS( const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType ) override;
	virtual void		UpdateTotalDemoTime( uint32 TimeInMS ) override;
	virtual void		UpdatePlaybackTime(uint32 TimeInMS) override {}
	virtual uint32		GetTotalDemoTime() const override { return TotalDemoTimeInMS; }
	virtual bool		IsDataAvailable() const override;
	virtual void		SetHighPriorityTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS ) override;
	virtual bool		IsDataAvailableForTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS ) override;
	virtual bool		IsLoadingCheckpoint() const override;
	virtual bool		IsLive() const override;
	virtual void		DeleteFinishedStream( const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate ) override;
	virtual void		DeleteFinishedStream( const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate ) override;
	virtual void		EnumerateStreams( const FNetworkReplayVersion& InReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate ) override;
	virtual void		EnumerateEvents( const FString& Group, const FEnumerateEventsCallback& Delegate ) override;
	virtual void		EnumerateEvents( const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate ) override;
	virtual void		EnumerateEvents( const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate ) override;
	virtual void		EnumerateRecentStreams( const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate ) override;
	virtual void		AddUserToReplay(const FString& UserString) override;
	virtual void		RequestEventData(const FString& EventId, const FRequestEventDataCallback& Delegate) override;
	virtual void		RequestEventData(const FString& ReplayName, const FString& EventId, const FRequestEventDataCallback& Delegate) override;
	virtual void		RequestEventData(const FString& ReplayName, const FString& EventId, const int32 UserIndex, const FRequestEventDataCallback& Delegate) override;
	virtual void		RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void		RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void		RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void		SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate) override;
	virtual void		KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate) override;
	virtual void		KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate) override;
	virtual void		RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate) override;
	virtual void		RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	virtual void		RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate) override;
	virtual void		RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;

	virtual FString		GetReplayID() const override { return SessionName; }
	virtual EReplayStreamerState GetReplayStreamerState() const override { return StreamerState; }
	virtual void		SetTimeBufferHintSeconds(const float InTimeBufferHintSeconds) override {}
	virtual void		RefreshHeader() override;
	virtual void		DownloadHeader(const FDownloadHeaderCallback& Delegate);
	virtual uint32		GetMaxFriendlyNameSize() const override { return 0; }

	virtual EStreamingOperationResult SetDemoPath(const FString& DemoPath) override
	{
		return EStreamingOperationResult::Unsupported;
	}

	virtual EStreamingOperationResult GetDemoPath(FString& DemoPath) const override
	{
		return EStreamingOperationResult::Unsupported;
	}

	virtual bool IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const override;

	/** FHttpNetworkReplayStreamer */
	void UploadHeader();
	void FlushStream();
	void ConditionallyFlushStream();
	void StopUploading();
	bool IsTaskPendingOrInFlight( const EQueuedHttpRequestType::Type Type ) const;
	void CancelInFlightOrPendingTask( const EQueuedHttpRequestType::Type Type );
	void ConditionallyDownloadNextChunk();
	void RefreshViewer( const bool bFinal );
	void ConditionallyRefreshViewer();
	
	UE_DEPRECATED(5.1, "No longer used")
	void SetLastError(const ENetworkReplayError::Type InLastError) { SetLastError(EHttpReplayResult::Unknown); }
	void SetLastError(FHttpReplayResult&& Result);

	virtual void CancelStreamingRequests();
	void FlushCheckpointInternal( uint32 TimeInMS );
	virtual void AddEvent( const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data ) override;
	virtual void AddOrUpdateEvent( const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data ) override;
	void AddRequestToQueue( const EQueuedHttpRequestType::Type Type, TSharedPtr<class IHttpRequest, ESPMode::ThreadSafe> Request, const int32 InMaxRetries = 0, const float InRetryDelay = 0.0f );
	void AddCustomRequestToQueue( TSharedPtr<FQueuedHttpRequest> Request );
	void AddResponseToCache( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse );
	void CleanupResponseCache();
	bool RetryRequest( TSharedPtr<FQueuedHttpRequest> Request, FHttpResponsePtr HttpResponse, const bool bIgnoreResponseCode = false );
	void EnumerateCheckpoints();
	void ConditionallyEnumerateCheckpoints();

	virtual void ProcessRequestInternal( TSharedPtr< class IHttpRequest, ESPMode::ThreadSafe > Request );
	virtual bool SupportsCompression() const { return false; }
	virtual bool CompressBuffer( const TArray< uint8 >& InBuffer, FHttpStreamFArchive& OutCompressed ) const { return false; }
	virtual bool DecompressBuffer(FHttpStreamFArchive& InCompressed, TArray< uint8 >& OutBuffer) const { return false; }

	virtual FString GetRecordingMetadata() const;
	virtual bool DecompressResponse(FHttpResponsePtr HttpResponse, TArray<uint8>& ResultBuffer) const;
	virtual bool CompressRequest(FHttpRequestPtr HttpRequest, const TArray<uint8>& RequestBuffer) const;

	void InternalGotoTimeInMS(const uint32 TimeInMS, const FGotoCallback& Delegate, bool bDelta);
	void InternalGotoCheckpointIndex(const int32 CheckpointIndex, const FGotoCallback& Delegate, const FHttpRequestCompleteDelegate& RequestDelegate);

	/** Delegates */
	void RequestFinished(EReplayStreamerState ExpectedStreamerState, EQueuedHttpRequestType::Type ExpectedType, FHttpRequestPtr HttpRequest );

	void HttpStartDownloadingFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpDownloadHeaderFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FDownloadHeaderCallback Delegate);
	void HttpDownloadFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, int32 RequestedStreamChunkIndex, bool bStreamWasLive );
	void HttpDownloadCheckpointFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpDownloadCheckpointDeltaFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpRefreshViewerFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpStartUploadingFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpStopUploadingFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpHeaderUploadFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpUploadStreamFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpUploadCheckpointFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpUploadCustomEventFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpEnumerateSessionsFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FEnumerateStreamsCallback Delegate );
	void HttpEnumerateCheckpointsFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpEnumerateEventsFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FEnumerateEventsCallback EnumerateEventsDelegate );
	void HttpAddUserFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded );
	void HttpRequestEventDataFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FRequestEventDataCallback RequestEventDataCompleteDelegate );

	// Purposefully passing a copy of a string here, as we call this from a delegate and don't want
	// to inadvertently capture a reference which may go out of scope.
	void KeepReplayFinished( FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, FKeepReplayCallback KeepReplayDelegate, FString ReplayName );

	bool ProcessNextHttpRequest();
	void Tick( const float DeltaTime );
	bool IsHttpRequestInFlight() const;		// True there is an http request currently in flight
	bool HasPendingHttpRequests() const;	// True if there is an http request in flight, or there are more to process
	bool IsStreaming() const;				// True if we are streaming a replay up or down

	FHttpStreamFArchive		HeaderArchive;			// Archive used to buffer the header stream
	FHttpStreamFArchive		StreamArchive;			// Archive used to buffer the data stream
	FHttpStreamFArchive		CheckpointArchive;		// Archive used to buffer checkpoint data
	FString					SessionName;			// Name of the session on the http replay server
	FNetworkReplayVersion	ReplayVersion;			// Version of the session
	FString					ServerURL;				// The address of the server
	int32					StreamChunkIndex;		// Used as a counter to increment the stream.x extension count
	double					LastChunkTime;			// The last time we uploaded/downloaded a chunk
	double					LastRefreshViewerTime;	// The last time we refreshed ourselves as an active viewer
	double					LastRefreshCheckpointTime;
	EReplayStreamerState	StreamerState;			// Overall state of the streamer
	bool					bStopStreamingCalled;
	bool					bStreamIsLive;			// If true, we are viewing a live stream
	FString					StreamMetadata;
	int32					NumTotalStreamChunks;
	uint32					TotalDemoTimeInMS;
	uint32					LastTotalDemoTimeInMS;
	uint32					StreamTimeRangeStart;
	uint32					StreamTimeRangeEnd;
	FString					ViewerName;
	uint32					HighPriorityEndTime;

	FStartStreamingCallback			StartStreamingDelegate;		// Delegate passed in to StartStreaming
	FGotoCallback					GotoCheckpointDelegate;
	int32							DownloadCheckpointIndex;
	int32							DeltaDownloadCheckpointIndex;
	int64							LastGotoTimeInMS;

	FReplayEventList					CheckpointList;

	TArray< TSharedPtr< FQueuedHttpRequest > >	QueuedHttpRequests;
	TSharedPtr< FQueuedHttpRequest >			InFlightHttpRequest;

	TSet< FString >						EventGroupSet;

	uint64								TotalUploadBytes;

	TMap< FString, FCachedResponse >	ResponseCache;

	int32								RefreshViewerFails;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

class FHttpNetworkReplayStreamingFactory : public INetworkReplayStreamingFactory, public FTickableGameObject
{
public:
	/** INetworkReplayStreamingFactory */
	UE_API virtual TSharedPtr< INetworkReplayStreamer > CreateReplayStreamer() override;

	/** FTickableGameObject */
	UE_API virtual void Tick( float DeltaTime ) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UE_API virtual TStatId GetStatId() const override;
	virtual bool IsTickableWhenPaused() const override { return true; }
	UE_API virtual void Flush() override;

	UE_API virtual void ShutdownModule() override;

	UE_API bool HasPendingHttpRequests() const;

	TArray< TSharedPtr< FHttpNetworkReplayStreamer > > HttpStreamers;
};

#undef UE_API
