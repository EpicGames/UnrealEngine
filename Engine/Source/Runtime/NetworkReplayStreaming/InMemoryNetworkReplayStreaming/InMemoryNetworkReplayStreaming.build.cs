// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InMemoryNetworkReplayStreaming : ModuleRules
	{
		public InMemoryNetworkReplayStreaming( ReadOnlyTargetRules Target ) : base(Target)
		{
			PublicIncludePathModuleNames.Add("Engine");

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"NetworkReplayStreaming",
                    "Json"
				}
			);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
