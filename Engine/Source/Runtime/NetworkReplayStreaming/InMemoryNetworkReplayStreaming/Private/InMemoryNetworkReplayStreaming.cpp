// Copyright Epic Games, Inc. All Rights Reserved.

#include "InMemoryNetworkReplayStreaming.h"
#include "Serialization/MemoryReader.h"
#include "Misc/Guid.h"
#include "Misc/DateTime.h"
#include "HAL/IConsoleManager.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"

DEFINE_LOG_CATEGORY_STATIC( LogMemoryReplay, Log, All );

static FString GetAutomaticDemoName()
{
	return FGuid::NewGuid().ToString();
}

void FInMemoryNetworkReplayStreamer::StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate)
{
	FStartStreamingResult Result;
	Result.bRecording = Params.bRecord;

	if ( Params.CustomName.IsEmpty() )
	{
		if ( Params.bRecord )
		{
			// If we're recording and the caller didn't provide a name, generate one automatically
			CurrentStreamName = GetAutomaticDemoName();
		}
		else
		{
			// Can't play a replay if the user didn't provide a name!
			Result.Result = EStreamingOperationResult::ReplayNotFound;
			Delegate.ExecuteIfBound(Result);
			return;
		}
	}
	else
	{
		CurrentStreamName = Params.CustomName;
	}

	if ( !Params.bRecord )
	{
		FInMemoryReplay* FoundReplay = GetCurrentReplay();
		if (FoundReplay == nullptr)
		{
			Result.Result = EStreamingOperationResult::ReplayNotFound;
			Delegate.ExecuteIfBound(Result);
			return;
		}

		FileAr.Reset(new FInMemoryReplayStreamArchive(FoundReplay->StreamChunks));
		FileAr->SetIsSaving(Params.bRecord);
		FileAr->SetIsLoading(!Params.bRecord);
		HeaderAr.Reset(new FMemoryReader(FoundReplay->Header));
		StreamerState = EReplayStreamerState::Playback;
	}
	else
	{
		// Add or overwrite a demo with this name
		TUniquePtr<FInMemoryReplay> NewReplay(new FInMemoryReplay);

		NewReplay->StreamInfo.Name = CurrentStreamName;
		NewReplay->StreamInfo.FriendlyName = Params.FriendlyName;
		NewReplay->StreamInfo.Timestamp = FDateTime::Now();
		NewReplay->StreamInfo.bIsLive = true;
		NewReplay->StreamInfo.Changelist = Params.ReplayVersion.Changelist;
		NewReplay->NetworkVersion = Params.ReplayVersion.NetworkVersion;

		// Open archives for writing
		FileAr.Reset(new FInMemoryReplayStreamArchive(NewReplay->StreamChunks));
		FileAr->SetIsSaving(Params.bRecord);
		FileAr->SetIsLoading(!Params.bRecord);
		HeaderAr.Reset(new FMemoryWriter(NewReplay->Header));

		OwningFactory->Replays.Add(CurrentStreamName, MoveTemp(NewReplay));
		
		StreamerState = EReplayStreamerState::Recording;
	}

	// Notify immediately
	if (FileAr.Get() != nullptr && HeaderAr.Get() != nullptr)
	{
		Result.Result = EStreamingOperationResult::Success;
	}
	
	Delegate.ExecuteIfBound(Result);
}

void FInMemoryNetworkReplayStreamer::StopStreaming()
{
	if (StreamerState == EReplayStreamerState::Recording)
	{
		FInMemoryReplay* FoundReplay = GetCurrentReplayChecked();

		FoundReplay->StreamInfo.SizeInBytes = FoundReplay->Header.Num() + FoundReplay->TotalStreamSize() + FoundReplay->Metadata.Num();
		for(const auto& Checkpoint : FoundReplay->Checkpoints)
		{
			FoundReplay->StreamInfo.SizeInBytes += Checkpoint.Data.Num();
		}

		FoundReplay->StreamInfo.bIsLive = false;
	}

	HeaderAr.Reset();
	FileAr.Reset();

	CurrentStreamName.Empty();
	StreamerState = EReplayStreamerState::Idle;
}

FArchive* FInMemoryNetworkReplayStreamer::GetHeaderArchive()
{
	return HeaderAr.Get();
}

FArchive* FInMemoryNetworkReplayStreamer::GetStreamingArchive()
{
	return FileAr.Get();
}

void FInMemoryNetworkReplayStreamer::UpdateTotalDemoTime(uint32 TimeInMS)
{
	check(StreamerState == EReplayStreamerState::Recording);

	FInMemoryReplay* FoundReplay = GetCurrentReplayChecked();

	FoundReplay->StreamInfo.LengthInMS = TimeInMS;
}

uint32 FInMemoryNetworkReplayStreamer::GetTotalDemoTime() const
{
	check(StreamerState != EReplayStreamerState::Idle);

	const FInMemoryReplay* FoundReplay = GetCurrentReplayChecked();

	return FoundReplay->StreamInfo.LengthInMS;
}

bool FInMemoryNetworkReplayStreamer::IsDataAvailable() const
{
	// Assumptions:
	// 1. All streamer instances run on the same thread, not simultaneously
	// 2. A recording DemoNetDriver will write either no frames or entire frames each time it ticks
	return StreamerState == EReplayStreamerState::Playback && FileAr.IsValid() && FileAr->Tell() < FileAr->TotalSize();
}

bool FInMemoryNetworkReplayStreamer::IsLive() const
{
	return IsNamedStreamLive(CurrentStreamName);
}

bool FInMemoryNetworkReplayStreamer::IsNamedStreamLive( const FString& StreamName ) const
{
	TUniquePtr<FInMemoryReplay>* FoundReplay = OwningFactory->Replays.Find(StreamName);
	if (FoundReplay != nullptr)
	{
		return (*FoundReplay)->StreamInfo.bIsLive;
	}

	return false;
}

void FInMemoryNetworkReplayStreamer::DeleteFinishedStream( const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate )
{
	DeleteFinishedStream(StreamName, Delegate);
}

void FInMemoryNetworkReplayStreamer::DeleteFinishedStream( const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate )
{
	// Danger! Deleting a stream that is still being read by another streaming instance is not supported!
	FDeleteFinishedStreamResult Result;

	// Live streams can't be deleted
	if (IsNamedStreamLive(StreamName))
	{
		UE_LOGF(LogMemoryReplay, Log, "Can't delete network replay stream %ls because it is live!", *StreamName);
	}
	else
	{
		if (OwningFactory->Replays.Remove(StreamName) > 0)
		{
			Result.Result = EStreamingOperationResult::Success;
		}
	}

	
	Delegate.ExecuteIfBound(Result);
}

void FInMemoryNetworkReplayStreamer::EnumerateRecentStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::EnumerateRecentStreams is currently unsupported.");
	FEnumerateStreamsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::EnumerateStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate)
{
	FEnumerateStreamsResult Result;
	Result.Result = EStreamingOperationResult::Success;

	for ( const auto& StreamPair : OwningFactory->Replays )
	{
		// Check version. NetworkVersion and changelist of 0 will ignore version check.
		const bool NetworkVersionMatches = ReplayVersion.NetworkVersion == StreamPair.Value->NetworkVersion;
		const bool ChangelistMatches = ReplayVersion.Changelist == StreamPair.Value->StreamInfo.Changelist;

		const bool NetworkVersionPasses = ReplayVersion.NetworkVersion == 0 || NetworkVersionMatches;
		const bool ChangelistPasses = ReplayVersion.Changelist == 0 || ChangelistMatches;

		if ( NetworkVersionPasses && ChangelistPasses )
		{
			Result.FoundStreams.Add( StreamPair.Value->StreamInfo );
		}
	}

	Delegate.ExecuteIfBound( Result );
}

void FInMemoryNetworkReplayStreamer::AddUserToReplay( const FString& UserString )
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::AddUserToReplay is currently unsupported.");
}

void FInMemoryNetworkReplayStreamer::AddEvent( const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data )
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::AddEvent is currently unsupported.");
}

void FInMemoryNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents(Group, Delegate);
}

void FInMemoryNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents(Group, Delegate);
}

void FInMemoryNetworkReplayStreamer::EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::EnumerateEvents is currently unsupported.");
	FEnumerateEventsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::RequestEventData is currently unsupported.");
	FRequestEventDataResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	RequestEventData(EventID, Delegate);
}

void FInMemoryNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	RequestEventData(EventID, Delegate);
}

void FInMemoryNetworkReplayStreamer::RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::RequestEventGroupData is currently unsupported.");
	FRequestEventGroupDataResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(Group, Delegate);
}

void FInMemoryNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(Group, Delegate);
}

void FInMemoryNetworkReplayStreamer::SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::SearchEvents is currently unsupported.");
	FSearchEventsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	KeepReplay(ReplayName, bKeep, Delegate);
}

void FInMemoryNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::KeepReplay is currently unsupported.");
	FKeepReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplayFriendlyName(ReplayName, NewFriendlyName, Delegate);
}

void FInMemoryNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::RenameReplayFriendlyName is currently unsupported.");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FInMemoryNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplay(ReplayName, NewName, Delegate);
}

void FInMemoryNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::RenameReplay is currently unsupported.");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

FArchive* FInMemoryNetworkReplayStreamer::GetCheckpointArchive()
{
	// If the archive is null, and the API is being used properly, the caller is writing a checkpoint...
	if ( CheckpointAr.Get() == nullptr )
	{
		check(StreamerState != EReplayStreamerState::Playback);

		UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::GetCheckpointArchive. Creating new checkpoint.");

		FInMemoryReplay* FoundReplay = GetCurrentReplayChecked();

		// Free old checkpoints and stream chunks that are older than the threshold.
		if (TimeBufferHintSeconds > 0.0f)
		{
			// Absolute time at which the buffer should start
			const uint32 BufferStartTimeMS = UE::LWC::FloatToIntCastChecked<uint32>(FoundReplay->StreamInfo.LengthInMS - (TimeBufferHintSeconds * 1000.0));

			// Store the found checkpoint's time so that we can line up chunks with it
			uint32 FoundCheckpointTime = 0;

			// Always keep at least one checkpoint
			int32 FirstCheckpointIndexToKeep = 0;

			// Go backwards through the checkpoints and find the one that is before the buffer starts.
			for (int32 i = FoundReplay->Checkpoints.Num() - 1; i >= 0; --i)
			{
				const FInMemoryReplay::FCheckpoint& Checkpoint = FoundReplay->Checkpoints[i];
				if (Checkpoint.TimeInMS <= BufferStartTimeMS)
				{
					FirstCheckpointIndexToKeep = i;
					FoundCheckpointTime = Checkpoint.TimeInMS;
					break;
				}
			}

			// Remove the checkpoints.
			const int32 NumCheckpointsToRemove = FirstCheckpointIndexToKeep;
			FoundReplay->Checkpoints.RemoveAt(0, NumCheckpointsToRemove, EAllowShrinking::No);

			// Always keep at least one chunk
			int32 FirstChunkIndexToKeep = 0;

			// Go backwards through the chunks and find the one that corresponds to the checkpoint we kept (or the beginning of the stream).
			for (int32 i = FoundReplay->StreamChunks.Num() - 1; i >= 0; --i)
			{
				const FInMemoryReplay::FStreamChunk& Chunk = FoundReplay->StreamChunks[i];
				if (Chunk.TimeInMS <= FoundCheckpointTime)
				{
					FirstChunkIndexToKeep = i;
					break;
				}
			}

			// Remove the chunks.
			const int32 NumChunksToRemove = FirstChunkIndexToKeep;
			FoundReplay->StreamChunks.RemoveAt(0, NumChunksToRemove, EAllowShrinking::No);
		}

		// Save to a temporary checkpoint that will moved onto the replay's checkpoint list in FlushCheckpoint().
		CheckpointCurrentlyBeingSaved.Reset();
		CheckpointAr.Reset(new FMemoryWriter(CheckpointCurrentlyBeingSaved.Data));
	}

	return CheckpointAr.Get();
}

void FInMemoryNetworkReplayStreamer::FlushCheckpoint(const uint32 TimeInMS)
{
	UE_LOGF(LogMemoryReplay, Log, "FInMemoryNetworkReplayStreamer::FlushCheckpoint. TimeInMS: %u", TimeInMS);

	check(FileAr.Get() != nullptr);
	check(CheckpointCurrentlyBeingSaved.Data.Num() != 0);

	// Finalize the checkpoint data.
	CheckpointAr.Reset();
	
	CheckpointCurrentlyBeingSaved.TimeInMS = TimeInMS;
	CheckpointCurrentlyBeingSaved.StreamByteOffset = IntCastChecked<uint32>(FileAr->Tell());

	FInMemoryReplay* const FoundReplay = GetCurrentReplayChecked();

	const int32 NewCheckpointIndex = FoundReplay->Checkpoints.Add(MoveTemp(CheckpointCurrentlyBeingSaved));

	// Start a new stream chunk for the new checkpoint
	FInMemoryReplay::FStreamChunk NewChunk;
	if (FoundReplay->StreamChunks.Num() > 0)
	{
		NewChunk.StartIndex = FoundReplay->StreamChunks.Last().StartIndex + FoundReplay->StreamChunks.Last().Data.Num();
		NewChunk.TimeInMS = FoundReplay->StreamInfo.LengthInMS;
	}
	FoundReplay->StreamChunks.Add(NewChunk);
}

void FInMemoryNetworkReplayStreamer::GotoCheckpointIndex(const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType)
{
	GotoCheckpointIndexInternal(CheckpointIndex, Delegate, -1);
}

void FInMemoryNetworkReplayStreamer::GotoCheckpointIndexInternal(int32 CheckpointIndex, const FGotoCallback& Delegate, int32 ExtraSkipTimeInMS)
{
	check( FileAr.Get() != nullptr);

	FGotoResult Result;
	
	if ( CheckpointIndex == -1 )
	{
		// Create a dummy checkpoint archive to indicate this is the first checkpoint
		CheckpointAr.Reset(new FArchive);

		FileAr->Seek(0);
		Result.ExtraTimeMS = ExtraSkipTimeInMS;
		Result.Result = EStreamingOperationResult::Success;
		Result.CheckpointInfo.CheckpointIndex = FReplayCheckpointInfo::NO_CHECKPOINT;
		Result.CheckpointInfo.CheckpointStartTime = FReplayCheckpointInfo::NO_CHECKPOINT;
	}
	else
	{
		FInMemoryReplay* FoundReplay = GetCurrentReplayChecked();

		if (!FoundReplay->Checkpoints.IsValidIndex(CheckpointIndex))
		{
			UE_LOGF(LogMemoryReplay, Log, "FNullNetworkReplayStreamer::GotoCheckpointIndex. Index %i is out of bounds.", CheckpointIndex);
		}
		else
		{
			const FInMemoryReplay::FCheckpoint& Checkpoint = FoundReplay->Checkpoints[CheckpointIndex];
			CheckpointAr.Reset(new FMemoryReader(Checkpoint.Data));

			FileAr->Seek(Checkpoint.StreamByteOffset);
			Result.ExtraTimeMS = ExtraSkipTimeInMS;
			Result.Result = EStreamingOperationResult::Success;
			Result.CheckpointInfo.CheckpointIndex = CheckpointIndex;
			Result.CheckpointInfo.CheckpointStartTime = Checkpoint.TimeInMS;
		}
	}

	Delegate.ExecuteIfBound( Result );
}

FInMemoryReplay* FInMemoryNetworkReplayStreamer::GetCurrentReplay()
{
	TUniquePtr<FInMemoryReplay>* FoundReplay = OwningFactory->Replays.Find(CurrentStreamName);
	if (FoundReplay != nullptr)
	{
		return FoundReplay->Get();
	}

	return nullptr;
}

FInMemoryReplay* FInMemoryNetworkReplayStreamer::GetCurrentReplay() const
{
	TUniquePtr<FInMemoryReplay>* FoundReplay = OwningFactory->Replays.Find(CurrentStreamName);
	if (FoundReplay != nullptr)
	{
		return FoundReplay->Get();
	}

	return nullptr;
}

FInMemoryReplay* FInMemoryNetworkReplayStreamer::GetCurrentReplayChecked()
{
	FInMemoryReplay* FoundReplay = GetCurrentReplay();
	check(FoundReplay != nullptr);
	
	return FoundReplay;
}

FInMemoryReplay* FInMemoryNetworkReplayStreamer::GetCurrentReplayChecked() const
{
	FInMemoryReplay* FoundReplay = GetCurrentReplay();
	check(FoundReplay != nullptr);
	
	return FoundReplay;
}

void FInMemoryNetworkReplayStreamer::GotoTimeInMS(const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType)
{
	int32 CheckpointIndex = -1;

	FInMemoryReplay* FoundReplay = GetCurrentReplayChecked();

	// Checkpoints are sorted by time. Look backwards through the array
	// to find the one immediately preceding the target time.
	for (int32 i = FoundReplay->Checkpoints.Num() - 1; i >= 0; --i)
	{
		if (FoundReplay->Checkpoints[i].TimeInMS <= TimeInMS)
		{
			CheckpointIndex = i;
			break;
		}
	}

	if (CheckpointIndex == -1)
	{
		// No checkpoint was found. We may be going the beginning of the stream without an explicit
		// checkpoint, but if the target time is before the start time of the first stream chunk,
		// the data was likely discarded due to the TimeBufferHintSeconds value and we can't do anything
		// except report an error.
		if (FoundReplay->StreamChunks.Num() == 0 || FoundReplay->StreamChunks[0].TimeInMS > TimeInMS)
		{
			Delegate.ExecuteIfBound(FGotoResult());
			return;
		}
	}

	int32 ExtraSkipTimeInMS = TimeInMS;

	if ( CheckpointIndex >= 0 )
	{
		// Subtract off checkpoint time so we pass in the leftover to the engine to fast forward through for the fine scrubbing part
		ExtraSkipTimeInMS = TimeInMS - FoundReplay->Checkpoints[ CheckpointIndex ].TimeInMS;
	}

	GotoCheckpointIndexInternal( CheckpointIndex, Delegate, ExtraSkipTimeInMS );
}

void FInMemoryNetworkReplayStreamer::Tick(float DeltaSeconds)
{
	
}

TStatId FInMemoryNetworkReplayStreamer::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FInMemoryNetworkReplayStreamer, STATGROUP_Tickables);
}

void FInMemoryReplayStreamArchive::Serialize(void* V, int64 Length) 
{
	if (IsLoading() )
	{
		if (Pos + Length > TotalSize())
		{
			SetError();
			return;
		}
		
		FInMemoryReplay::FStreamChunk* CurrentChunk = GetCurrentChunk();

		if (CurrentChunk == nullptr)
		{
			SetError();
			return;
		}

		const int64 OffsetIntoChunk = Pos - CurrentChunk->StartIndex;
		FMemory::Memcpy(V, CurrentChunk->Data.GetData() + OffsetIntoChunk, Length);

		Pos += Length;
	}
	else
	{
		check(Pos <= TotalSize());

		FInMemoryReplay::FStreamChunk* CurrentChunk = GetCurrentChunk();

		if (CurrentChunk == nullptr)
		{
			SetError();
			return;
		}

		const int64 OffsetIntoChunk = Pos - CurrentChunk->StartIndex;
		const int32 SpaceNeeded = IntCastChecked<int32>(Length - (CurrentChunk->Data.Num() - OffsetIntoChunk));
		if (SpaceNeeded > 0)
		{
			CurrentChunk->Data.AddZeroed(SpaceNeeded);
		}

		FMemory::Memcpy(CurrentChunk->Data.GetData() + OffsetIntoChunk, V, Length);

		Pos += Length;
	}
}

int64 FInMemoryReplayStreamArchive::Tell() 
{
	return Pos;
}

int64 FInMemoryReplayStreamArchive::TotalSize()
{
	if (Chunks.Num() == 0)
	{
		return 0;
	}

	return Chunks.Last().StartIndex + Chunks.Last().Data.Num();
}

void FInMemoryReplayStreamArchive::Seek(int64 InPos) 
{
	check(InPos <= TotalSize());

	Pos = InPos;
}

bool FInMemoryReplayStreamArchive::AtEnd() 
{
	return Pos >= TotalSize();
}

FInMemoryReplay::FStreamChunk* FInMemoryReplayStreamArchive::GetCurrentChunk() const
{
	// This assumes that the Chunks array is always sorted by StartOffset!
	for (int i = Chunks.Num() - 1; i >= 0; --i)
	{
		if (Chunks[i].StartIndex <= Pos)
		{
			check(Chunks[i].StartIndex + Chunks[i].Data.Num() >= Pos);
			return &Chunks[i];
		}
	}

	return nullptr;
}

IMPLEMENT_MODULE(FInMemoryNetworkReplayStreamingFactory, InMemoryNetworkReplayStreaming)

TSharedPtr<INetworkReplayStreamer> FInMemoryNetworkReplayStreamingFactory::CreateReplayStreamer() 
{
	return TSharedPtr<INetworkReplayStreamer>(new FInMemoryNetworkReplayStreamer(this));
}
