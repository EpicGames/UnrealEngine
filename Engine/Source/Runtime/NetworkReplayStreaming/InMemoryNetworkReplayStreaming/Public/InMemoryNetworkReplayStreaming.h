// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Containers/UnrealString.h"
#include "NetworkReplayStreaming.h"
#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"
#include "Serialization/JsonSerializer.h"
#include "Tickable.h"

class FInMemoryNetworkReplayStreamingFactory;
class FNetworkReplayVersion;

/* Holds all data about an entire replay */
struct FInMemoryReplay
{
	struct FCheckpoint
	{
		FCheckpoint() :
			TimeInMS(0),
			StreamByteOffset(0) {}

		TArray<uint8> Data;
		uint32 TimeInMS;
		uint32 StreamByteOffset;
		
		void Reset()
		{
			Data.Reset();
			TimeInMS = 0;
			StreamByteOffset = 0;
		}
	};

	/** Represents a chunk of replay stream data between two checkpoints. */
	struct FStreamChunk
	{
		FStreamChunk() : StartIndex(0), TimeInMS(0) {}

		int32 StartIndex;
		uint32 TimeInMS;
		TArray<uint8> Data;
	};

	FInMemoryReplay() :
		NetworkVersion(0)
	{
		// Add the first stream chunk, this one will contain the stream before the first checkpoint is written. */
		StreamChunks.Emplace();
	}

	TArray<uint8> Header;
	TArray<FStreamChunk> StreamChunks;
	TArray<uint8> Metadata;
	TArray<FCheckpoint> Checkpoints;
	FNetworkReplayStreamInfo StreamInfo;
	uint32 NetworkVersion;

	int64 TotalStreamSize() const
	{
		int64 TotalSize = sizeof(FInMemoryReplay);
		
		TotalSize += Header.GetAllocatedSize();
		TotalSize += StreamChunks.GetAllocatedSize();

		for (const FStreamChunk& Chunk : StreamChunks)
		{
			TotalSize += Chunk.Data.GetAllocatedSize();
		}

		TotalSize += Metadata.GetAllocatedSize();
		TotalSize += Checkpoints.GetAllocatedSize();

		for (const FCheckpoint& Checkpoint : Checkpoints)
		{
			TotalSize += Checkpoint.Data.GetAllocatedSize();
		}

		return TotalSize;
	}
};

/**
 * An archive that handles the in-memory replay stream being divided into multiple chunks,
 * and earlier chunks being dropped when FInMemoryNetworkReplayStreamer::NumCheckpointsToKeep is set.
 * Assumes that that a single Serialize() call will not need to span multiple chunks!
 */
class FInMemoryReplayStreamArchive : public FArchive
{
public:
	FInMemoryReplayStreamArchive(TArray<FInMemoryReplay::FStreamChunk>& InChunks)
		: Pos(0)
		, Chunks(InChunks)
	{}

	virtual void	Serialize(void* V, int64 Length);
	virtual int64	Tell();
	virtual int64	TotalSize();
	virtual void	Seek(int64 InPos);
	virtual bool	AtEnd();

private:
	FInMemoryReplay::FStreamChunk* GetCurrentChunk() const;

	int64 Pos;
	TArray<FInMemoryReplay::FStreamChunk>& Chunks;
};

/** Streamer that keeps all data in memory only */
class FInMemoryNetworkReplayStreamer : public INetworkReplayStreamer, public FTickableGameObject
{
public:
	FInMemoryNetworkReplayStreamer(FInMemoryNetworkReplayStreamingFactory* InFactory) :
		OwningFactory(InFactory),
		StreamerState(EReplayStreamerState::Idle),
		TimeBufferHintSeconds(-1.0f)
	{
		check(OwningFactory != nullptr);
	}
	
	/** INetworkReplayStreamer implementation */
	virtual void StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate) override;
	virtual void StopStreaming() override;
	virtual FArchive* GetHeaderArchive() override;
	virtual FArchive* GetStreamingArchive() override;
	virtual FArchive* GetCheckpointArchive() override;
	virtual void FlushCheckpoint( const uint32 TimeInMS ) override;
	virtual void GotoCheckpointIndex( const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType ) override;
	virtual void GotoTimeInMS( const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType ) override;
	virtual void UpdateTotalDemoTime( uint32 TimeInMS ) override;
	virtual void UpdatePlaybackTime(uint32 TimeInMS) override {}
	virtual uint32 GetTotalDemoTime() const override;
	virtual bool IsDataAvailable() const override;
	virtual void SetHighPriorityTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS ) override { }
	virtual bool IsDataAvailableForTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS ) override { return true; }
	virtual bool IsLoadingCheckpoint() const override { return false; }
	virtual bool IsLive() const override;
	virtual void DeleteFinishedStream( const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate) override;
	virtual void DeleteFinishedStream( const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate ) override;
	virtual void EnumerateStreams( const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate ) override;
	virtual void EnumerateRecentStreams( const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate ) override;
	virtual void AddUserToReplay(const FString& UserString) override;
	virtual void AddEvent(const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data) override;
	virtual void AddOrUpdateEvent( const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data ) override {}
	virtual void EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate) override;
	virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate ) override;
	virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate ) override;
	virtual void RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	virtual void RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	virtual void RequestEventData(const FString& ReplayName, const FString& EventId, const int32 UserIndex, const FRequestEventDataCallback& Delegate) override;
	virtual void RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate) override;
	virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate) override;
	virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate) override;
	virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate) override;
	virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate) override;
	virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	virtual FString	GetReplayID() const override { return CurrentStreamName; }
	virtual EReplayStreamerState GetReplayStreamerState() const override { return StreamerState; }
	virtual void SetTimeBufferHintSeconds(const float InTimeBufferHintSeconds) override { TimeBufferHintSeconds = InTimeBufferHintSeconds; }
	virtual void RefreshHeader() override {};
	virtual void DownloadHeader(const FDownloadHeaderCallback& Delegate) override
	{
		FDownloadHeaderResult Result;
		Result.Result = EStreamingOperationResult::Success;
		Delegate.Execute(Result);
	}
	/** FTickableObjectBase implementation */
	virtual void Tick(float DeltaSeconds) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;

	/** FTickableGameObject implementation */
	virtual bool IsTickableWhenPaused() const override { return true; }

	virtual uint32 GetMaxFriendlyNameSize() const override { return 0; }

	virtual EStreamingOperationResult SetDemoPath(const FString& DemoPath) override
	{
		return EStreamingOperationResult::Unsupported;
	}

	virtual EStreamingOperationResult GetDemoPath(FString& DemoPath) const override
	{
		return EStreamingOperationResult::Unsupported;
	}

	virtual bool IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const override { return (CheckpointType == EReplayCheckpointType::Full); }

private:
	bool IsNamedStreamLive( const FString& StreamName ) const;

	/** Handles the details of loading a checkpoint */
	void GotoCheckpointIndexInternal(int32 CheckpointIndex, const FGotoCallback& Delegate, int32 TimeInMS);

	/**
	 * Returns a pointer to the currently active (recording or playback) replay in the owning factory's map.
	 * May return null if if the streamer state is idle.
	 */
	FInMemoryReplay* GetCurrentReplay();

	/**
	 * Returns a pointer to the currently active (recording or playback) replay in the owning factory's map.
	 * May return null if if the streamer state is idle.
	 */
	FInMemoryReplay* GetCurrentReplay() const;

	/**
	 * Returns a pointer to the currently active (recording or playback) replay in the owning factory's map.
	 * Asserts if the returned replay pointer would be null.
	 */
	FInMemoryReplay* GetCurrentReplayChecked();

	/**
	 * Returns a pointer to the currently active (recording or playback) replay in the owning factory's map.
	 * Asserts if the returned replay pointer would be null.
	 */
	FInMemoryReplay* GetCurrentReplayChecked() const;

	/** Pointer to the factory that owns this streamer instance */
	FInMemoryNetworkReplayStreamingFactory* OwningFactory;

	/** Handle to the archive that will read/write the demo header */
	TUniquePtr<FArchive> HeaderAr;

	/** Handle to the archive that will read/write network packets */
	TUniquePtr<FArchive> FileAr;

	/* Handle to the archive that will read/write checkpoint files */
	TUniquePtr<FArchive> CheckpointAr;

	/**
	 * Temporary checkpoint used during recording. Will be moved onto the
	 * replay's checkpoint list in FlushCheckpoint to commit it.
	 */
	FInMemoryReplay::FCheckpoint CheckpointCurrentlyBeingSaved;

	/** Overall state of the streamer */
	EReplayStreamerState StreamerState;

	/** Remember the name of the current stream, if any. */
	FString CurrentStreamName;

	/**
	 * If greater than zero, checkpoints that wouldn't be needed to rewind farther than this value,
	 * and the stream data associated with them, will be freed periodically to help limit memory usage.
	 */
	float TimeBufferHintSeconds;
};

class FInMemoryNetworkReplayStreamingFactory : public INetworkReplayStreamingFactory
{
public:
	virtual TSharedPtr< INetworkReplayStreamer > CreateReplayStreamer();

	friend class FInMemoryNetworkReplayStreamer;

private:
	TMap<FString, TUniquePtr<FInMemoryReplay>> Replays;
};
