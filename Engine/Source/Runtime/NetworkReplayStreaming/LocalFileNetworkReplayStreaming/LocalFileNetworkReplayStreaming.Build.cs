// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LocalFileNetworkReplayStreaming : ModuleRules
    {
        public LocalFileNetworkReplayStreaming(ReadOnlyTargetRules Target) : base(Target)
        {
			ShortName = "LFNRS";
			PrivateDependencyModuleNames.AddRange(
                new string[]
				{
					"Core",
					"CoreUObject",
                    "Engine",
					"NetworkReplayStreaming",
					"Json"
				} );

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
