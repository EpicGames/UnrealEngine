// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalFileNetworkReplayStreaming.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Async/Async.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "GameFramework/OnlineReplStructs.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Online/CoreOnline.h"
#include "ReplayHelper.h"
#include "Serialization/ArchiveCountMem.h"
#include "Serialization/LargeMemoryReader.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IPlatformFileManagedStorageWrapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LocalFileNetworkReplayStreaming)

DEFINE_LOG_CATEGORY_STATIC(LogLocalFileReplay, Log, All);

DECLARE_STATS_GROUP(TEXT("LocalReplay"), STATGROUP_LocalReplay, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Local replay compress time"), STAT_LocalReplay_CompressTime, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay decompress time"), STAT_LocalReplay_DecompressTime, STATGROUP_LocalReplay);

DECLARE_CYCLE_STAT(TEXT("Local replay encrypt time"), STAT_LocalReplay_EncryptTime, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay decrypt time"), STAT_LocalReplay_DecryptTime, STATGROUP_LocalReplay);

DECLARE_CYCLE_STAT(TEXT("Local replay read info"), STAT_LocalReplay_ReadReplayInfo, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay write info"), STAT_LocalReplay_WriteReplayInfo, STATGROUP_LocalReplay);

DECLARE_CYCLE_STAT(TEXT("Local replay rename"), STAT_LocalReplay_Rename, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay rename friendly"), STAT_LocalReplay_RenameFriendly, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay enumerate"), STAT_LocalReplay_Enumerate, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay delete"), STAT_LocalReplay_Delete, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay automatic name"), STAT_LocalReplay_AutomaticName, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay start recording"), STAT_LocalReplay_StartRecording, STATGROUP_LocalReplay);

DECLARE_CYCLE_STAT(TEXT("Local replay read checkpoint"), STAT_LocalReplay_ReadCheckpoint, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay read stream"), STAT_LocalReplay_ReadStream, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay read header"), STAT_LocalReplay_ReadHeader, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay read event"), STAT_LocalReplay_ReadEvent, STATGROUP_LocalReplay);

DECLARE_CYCLE_STAT(TEXT("Local replay flush checkpoint"), STAT_LocalReplay_FlushCheckpoint, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay flush stream"), STAT_LocalReplay_FlushStream, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay flush header"), STAT_LocalReplay_FlushHeader, STATGROUP_LocalReplay);
DECLARE_CYCLE_STAT(TEXT("Local replay flush event"), STAT_LocalReplay_FlushEvent, STATGROUP_LocalReplay);

namespace UE::Net::LocalFileReplay
{
	TAutoConsoleVariable<int32> CVarMaxCacheSize(TEXT("localReplay.MaxCacheSize"), 1024 * 1024 * 10, TEXT(""));
	TAutoConsoleVariable<int32> CVarMaxBufferedStreamChunks(TEXT("localReplay.MaxBufferedStreamChunks"), 10, TEXT(""));
	TAutoConsoleVariable<int32> CVarAllowLiveStreamDelete(TEXT("localReplay.AllowLiveStreamDelete"), 1, TEXT(""));
	TAutoConsoleVariable<float> CVarChunkUploadDelayInSeconds(TEXT("localReplay.ChunkUploadDelayInSeconds"), 20.0f, TEXT(""));

#if !UE_BUILD_SHIPPING
	TAutoConsoleVariable<int32> CVarAllowEncryptedRecording(TEXT("localReplay.AllowEncryptedRecording"), 1, TEXT(""));
#endif

	TAutoConsoleVariable<int32> CVarReplayRecordingMinSpace(TEXT("localReplay.ReplayRecordingMinSpace"), 20 * (1024 * 1024), TEXT("Minimum space needed to start recording a replay."));
	TAutoConsoleVariable<float> CVarMinLoadNextChunkDelaySeconds(TEXT("localReplay.MinLoadNextChunkDelaySeconds"), 3.0f, TEXT("Minimum time to wait between conditional chunk loads."));

	constexpr int32 MaxEncryptionKeySizeBytes = 4096;

	TAutoConsoleVariable<int32> CVarMaxFriendlySerializeBytes(TEXT("localReplay.MaxFriendlySerializeBytes"), 64 * 1024, TEXT("Maximum allowed serialized bytes when reading friendly name from file header."));
	
	TAutoConsoleVariable<bool> CVarbUsePlayerAccountIdPostfix(
		TEXT("localReplay.UsePlayerAccountIdPostfix"),
#if UE_BUILD_SHIPPING
		false,
#else
		true,
#endif
		TEXT("If true, the local replay filenames have the local player's accountId as a postfix. Useful to spectate multiple clients in the same device."),
		ECVF_Cheat
	);
};

const uint32 FLocalFileNetworkReplayStreamer::FileMagic = 0x1CA2E27F;
const uint32 FLocalFileNetworkReplayStreamer::MaxFriendlyNameLen = 256;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const uint32 FLocalFileNetworkReplayStreamer::LatestVersion = FLocalFileReplayCustomVersion::LatestVersion;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FOnLocalFileReplayFinishedWriting FLocalFileNetworkReplayStreamer::OnReplayFinishedWriting;

const FGuid FLocalFileReplayCustomVersion::Guid =  FGuid(0x95A4f03E, 0x7E0B49E4, 0xBA43D356, 0x94FF87D9);
FCustomVersionRegistration GRegisterLocalFileReplayCustomVersion(FLocalFileReplayCustomVersion::Guid, FLocalFileReplayCustomVersion::LatestVersion, TEXT("LocalFileReplay"));

FLocalFileNetworkReplayStreamer::FLocalFileSerializationInfo::FLocalFileSerializationInfo() 
	: FileVersion(FLocalFileReplayCustomVersion::CustomVersions)
{
	FileCustomVersions.SetVersion(FLocalFileReplayCustomVersion::Guid, FLocalFileReplayCustomVersion::LatestVersion, TEXT("LocalFileReplay"));
}

FLocalFileReplayCustomVersion::Type FLocalFileNetworkReplayStreamer::FLocalFileSerializationInfo::GetLocalFileReplayVersion() const
{
	if (const FCustomVersion* CustomVer = FileCustomVersions.GetVersion(FLocalFileReplayCustomVersion::Guid))
	{
		return (FLocalFileReplayCustomVersion::Type)CustomVer->Version;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return (FLocalFileReplayCustomVersion::Type)FileVersion;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FLocalFileEventInfo::CountBytes(FArchive& Ar) const
{
	Id.CountBytes(Ar);
	Group.CountBytes(Ar);
	Metadata.CountBytes(Ar);
}

void FLocalFileReplayInfo::CountBytes(FArchive& Ar) const
{
	FriendlyName.CountBytes(Ar);
	EncryptionKey.CountBytes(Ar);
	
	Chunks.CountBytes(Ar);
	DataChunks.CountBytes(Ar);

	Checkpoints.CountBytes(Ar);
	for (const FLocalFileEventInfo& Info : Checkpoints)
	{
		Info.CountBytes(Ar);
	}

	Events.CountBytes(Ar);
	for (const FLocalFileEventInfo& Info : Events)
	{
		Info.CountBytes(Ar);
	}
}

void FLocalFileStreamFArchive::Serialize(void* V, int64 Length) 
{
	if (IsLoading())
	{
		if ((Length < 0) || (ArchivePos + Length) > Buffer.Num())
		{
			UE_LOG(LogLocalFileReplay, Error, TEXT("FLocalFileStreamFArchive::Serialize: Attempted to serialize past end of archive: Position = %" INT64_FMT ", Size=%i, Requested = %" INT64_FMT), ArchivePos, Buffer.Num(), Length);
			SetError();
			return;
		}

		FMemory::Memcpy(V, Buffer.GetData() + ArchivePos, Length);

		ArchivePos += Length;
	}
	else
	{
		check(ArchivePos <= Buffer.Num());

		const int32 SpaceNeeded = IntCastChecked<int32>(Length - (Buffer.Num() - ArchivePos));
		if (SpaceNeeded > 0)
		{
			Buffer.AddUninitialized(SpaceNeeded);
		}

		FMemory::Memcpy(Buffer.GetData() + ArchivePos, V, Length);

		ArchivePos += Length;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Pos = static_cast<int32>(ArchivePos);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int64 FLocalFileStreamFArchive::Tell() const
{
	return ArchivePos;
}

int64 FLocalFileStreamFArchive::Tell()
{
	return ArchivePos;
}


int64 FLocalFileStreamFArchive::TotalSize() const
{
	return Buffer.Num();
}

int64 FLocalFileStreamFArchive::TotalSize()
{
	return Buffer.Num();
}

void FLocalFileStreamFArchive::Seek(int64 InPos) 
{
	check(InPos <= Buffer.Num());

	ArchivePos = InPos;
}

bool FLocalFileStreamFArchive::AtEnd() 
{
	return ArchivePos >= Buffer.Num() && bAtEndOfReplay;
}

FLocalFileNetworkReplayStreamer::FLocalFileNetworkReplayStreamer() 
	: StreamTimeRange(0, 0)
	, StreamDataOffset(0)
	, StreamChunkIndex(0)
	, LastChunkTime(0)
	, LastRefreshTime(0)
	, bStopStreamingCalled(false)
	, HighPriorityEndTime(0)
	, LastGotoTimeInMS(-1)
	, StreamerState(EReplayStreamerState::Idle)
	, DemoSavePath(GetDefaultDemoSavePath())
	, bCacheFileReadsInMemory(false)
{
}

FLocalFileNetworkReplayStreamer::FLocalFileNetworkReplayStreamer(const FString& InDemoSavePath) 
	: StreamTimeRange(0, 0)
	, StreamDataOffset(0)
	, StreamChunkIndex(0)
	, LastChunkTime(0)
	, LastRefreshTime(0)
	, bStopStreamingCalled(false)
	, HighPriorityEndTime(0)
	, LastGotoTimeInMS(-1)
	, StreamerState(EReplayStreamerState::Idle)
	, DemoSavePath(InDemoSavePath.EndsWith(TEXT("/")) ? InDemoSavePath : InDemoSavePath + FString("/"))
	, bCacheFileReadsInMemory(false)
{
}

FLocalFileNetworkReplayStreamer::~FLocalFileNetworkReplayStreamer()
{

}

bool FLocalFileNetworkReplayStreamer::ReadReplayInfo(const FString& StreamName, FLocalFileReplayInfo& Info, EReadReplayInfoFlags Flags) const
{
	SCOPE_CYCLE_COUNTER(STAT_LocalReplay_ReadReplayInfo);

	TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(GetDemoFullFilename(StreamName));
	if (LocalFileReader.IsValid())
	{
		return ReadReplayInfo(*LocalFileReader.Get(), Info, Flags);
	}

	return false;
}

bool FLocalFileNetworkReplayStreamer::ReadReplayInfo(FArchive& Archive, FLocalFileReplayInfo& Info, EReadReplayInfoFlags Flags) const
{
	FLocalFileSerializationInfo DefaultSerializationInfo;
	return ReadReplayInfo(Archive, Info, DefaultSerializationInfo, Flags);
}

bool FLocalFileNetworkReplayStreamer::ReadReplayInfo(FArchive& Archive, FLocalFileReplayInfo& Info, FLocalFileSerializationInfo& SerializationInfo, EReadReplayInfoFlags Flags) const
{
	// reset the info before reading
	Info = FLocalFileReplayInfo();

	if (Archive.TotalSize() != 0)
	{
		uint32 MagicNumber;
		Archive << MagicNumber;

		uint32 DoNotUse_FileVersion;
		Archive << DoNotUse_FileVersion;

		if (MagicNumber == FLocalFileNetworkReplayStreamer::FileMagic)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FCustomVersionContainer FileCustomVersions;

			if (DoNotUse_FileVersion >= FLocalFileReplayCustomVersion::CustomVersions)
			{
				FileCustomVersions.Serialize(Archive);

				TArray<FCustomVersionDifference> VersionDiffs = FCurrentCustomVersions::Compare(FileCustomVersions.GetAllVersions(), TEXT("LocalFileReplay"));
				for (const FCustomVersionDifference& Diff : VersionDiffs)
				{
					if (Diff.Type == ECustomVersionDifference::Missing)
					{
						UE_LOGF(LogLocalFileReplay, Error, "Replay was saved with a custom version that is not present. Tag %ls Version %d", *Diff.Version->Key.ToString(), Diff.Version->Version);
						Archive.SetError();
						return false;
					}
					else if (Diff.Type == ECustomVersionDifference::Invalid)
					{
						UE_LOGF(LogLocalFileReplay, Error, "Replay was saved with an invalid custom version. Tag %ls Version %d", *Diff.Version->Key.ToString(), Diff.Version->Version);
						Archive.SetError();
						return false;
					}
					else if (Diff.Type == ECustomVersionDifference::Newer)
					{
						const FCustomVersion MaxExpectedVersion = FCurrentCustomVersions::Get(Diff.Version->Key).GetValue();

						UE_LOGF(LogLocalFileReplay, Error, "Replay was saved with a newer custom version than the current. Tag %ls Name '%ls' ReplayVersion %d  MaxExpected %d",
							*Diff.Version->Key.ToString(), *MaxExpectedVersion.GetFriendlyName().ToString(), Diff.Version->Version, MaxExpectedVersion.Version);
						Archive.SetError();
						return false;
					}
				}

				Archive.SetCustomVersions(FileCustomVersions);
			}
			else
			{
				Archive.SetCustomVersion(FLocalFileReplayCustomVersion::Guid, DoNotUse_FileVersion, TEXT("LocalFileReplay"));
				FileCustomVersions.SetVersion(FLocalFileReplayCustomVersion::Guid, DoNotUse_FileVersion, TEXT("LocalFileReplay"));
			}

			SerializationInfo.FileVersion = DoNotUse_FileVersion;
			SerializationInfo.FileCustomVersions = FileCustomVersions;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// read summary info
			Archive << Info.LengthInMS;
			Archive << Info.NetworkVersion;
			Archive << Info.Changelist;

			FString FriendlyName;
			{
				TGuardValue<int64> MaxSerialize(Archive.ArMaxSerializeSize, UE::Net::LocalFileReplay::CVarMaxFriendlySerializeBytes.GetValueOnAnyThread());
				Archive << FriendlyName;
			}

			if (Archive.IsError())
			{
				UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Failed to serialize replay friendly name.");
				return false;
			}

			SerializationInfo.FileFriendlyName = FriendlyName;

			if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::FixedSizeFriendlyName)
			{
				// trim whitespace since this may have been padded
				Info.FriendlyName = FriendlyName.TrimEnd();
			}
			else
			{
				// Note, don't touch the FriendlyName if this is an older replay.
				// Users can adjust the name as necessary using GetMaxFriendlyNameSize.
				UE_LOGF(LogLocalFileReplay, Warning, "ReadReplayInfo - Loading an old replay, friendly name length **must not** be changed.");
			}

			uint32 IsLive;
			Archive << IsLive;

			Info.bIsLive = (IsLive != 0);

			if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::RecordingTimestamp)
			{
				Archive << Info.Timestamp;
			}

			if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::CompressionSupport)
			{
				uint32 Compressed;
				Archive << Compressed;

				Info.bCompressed = (Compressed != 0);
			}

			if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::EncryptionSupport)
			{
				uint32 Encrypted;
				Archive << Encrypted;

				Info.bEncrypted = (Encrypted != 0);

				const int64 KeyPos = Archive.Tell();

				int32 KeySize;
				Archive << KeySize;

				if (KeySize >= 0 && KeySize <= UE::Net::LocalFileReplay::MaxEncryptionKeySizeBytes)
				{
					Archive.Seek(KeyPos);

					Archive << Info.EncryptionKey;
				}
				else
				{
					UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Serialized an invalid encryption key size: %d", KeySize);
					Archive.SetError();
					return false;
				}
			}

			if (!Info.bIsLive && Info.bEncrypted && (Info.EncryptionKey.Num() == 0))
			{
				UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Completed replay is marked encrypted but has no key!");
				Archive.SetError();
				return false;
			}

			int64 TotalSize = Archive.TotalSize();

			// now look for all chunks
			while (!Archive.AtEnd())
			{
				int64 TypeOffset = Archive.Tell();

				ELocalFileChunkType ChunkType;
				Archive << ChunkType;

				int32 Idx = Info.Chunks.AddDefaulted();
				FLocalFileChunkInfo& Chunk = Info.Chunks[Idx];
				Chunk.ChunkType = ChunkType;

				Archive << Chunk.SizeInBytes;

				Chunk.TypeOffset = TypeOffset;
				Chunk.DataOffset = Archive.Tell();

				if ((Chunk.SizeInBytes < 0) || ((Chunk.DataOffset + Chunk.SizeInBytes) > TotalSize))
				{
					UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Invalid chunk size: %d", Chunk.SizeInBytes);
					Archive.SetError();
					return false;
				}

				switch(ChunkType)
				{
				case ELocalFileChunkType::Header:
				{
					if (Info.HeaderChunkIndex == INDEX_NONE)
					{
						Info.HeaderChunkIndex = Idx;
					}
					else
					{
						UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Found multiple header chunks");
						Archive.SetError();
						return false;
					}
				}
				break;
				case ELocalFileChunkType::Checkpoint:
				{
					int32 CheckpointIdx = Info.Checkpoints.AddDefaulted();
					FLocalFileEventInfo& Checkpoint = Info.Checkpoints[CheckpointIdx];

					Checkpoint.ChunkIndex = Idx;
					
					Archive << Checkpoint.Id;
					Archive << Checkpoint.Group;
					Archive << Checkpoint.Metadata;
					Archive << Checkpoint.Time1;
					Archive << Checkpoint.Time2;

					Archive << Checkpoint.SizeInBytes;

					Checkpoint.EventDataOffset = Archive.Tell();
					
					if ((Checkpoint.SizeInBytes < 0) || ((Checkpoint.EventDataOffset + Checkpoint.SizeInBytes) > TotalSize))
					{
						UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Invalid checkpoint disk size: %d", Checkpoint.SizeInBytes);
						Archive.SetError();
						return false;
					}
				}
				break;
				case ELocalFileChunkType::ReplayData:
				{
					int32 DataIdx = Info.DataChunks.AddDefaulted();
					FLocalFileReplayDataInfo& DataChunk = Info.DataChunks[DataIdx];
					DataChunk.ChunkIndex = Idx;
					DataChunk.StreamOffset = Info.TotalDataSizeInBytes;

					if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::StreamChunkTimes)
					{
						Archive << DataChunk.Time1;
						Archive << DataChunk.Time2;
						Archive << DataChunk.SizeInBytes;
					}
					else
					{
						DataChunk.SizeInBytes = Chunk.SizeInBytes;
					}

					if (SerializationInfo.GetLocalFileReplayVersion() < FLocalFileReplayCustomVersion::EncryptionSupport)
					{
						DataChunk.ReplayDataOffset = Archive.Tell();
	
						if (Info.bCompressed)
						{
							DataChunk.MemorySizeInBytes = GetDecompressedSizeBackCompat(Archive);
						}
						else
						{
							DataChunk.MemorySizeInBytes = DataChunk.SizeInBytes;
						}
					}
					else
					{
						Archive << DataChunk.MemorySizeInBytes;

						DataChunk.ReplayDataOffset = Archive.Tell();
					}

					if ((DataChunk.SizeInBytes < 0) || ((DataChunk.ReplayDataOffset + DataChunk.SizeInBytes) > TotalSize))
					{
						UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Invalid stream chunk disk size: %d", DataChunk.SizeInBytes);
						Archive.SetError();
						return false;
					}

					if (DataChunk.MemorySizeInBytes < 0)
					{
						UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Invalid stream chunk memory size: %d", DataChunk.MemorySizeInBytes);
						Archive.SetError();
						return false;
					}

					Info.TotalDataSizeInBytes += DataChunk.MemorySizeInBytes;
				}
				break;
				case ELocalFileChunkType::Event:
				{
					int32 EventIdx = Info.Events.AddDefaulted();
					FLocalFileEventInfo& Event = Info.Events[EventIdx];
					Event.ChunkIndex = Idx;

					Archive << Event.Id;
					Archive << Event.Group;
					Archive << Event.Metadata;
					Archive << Event.Time1;
					Archive << Event.Time2;

					Archive << Event.SizeInBytes;

					Event.EventDataOffset = Archive.Tell();

					if ((Event.SizeInBytes < 0) || ((Event.EventDataOffset + Event.SizeInBytes) > TotalSize))
					{
						UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Invalid event disk size: %d", Event.SizeInBytes);
						Archive.SetError();
						return false;
					}
				}
				break;
				case ELocalFileChunkType::Unknown:
					UE_LOGF(LogLocalFileReplay, Verbose, "ReadReplayInfo: Skipping unknown (cleared) chunk");
					break;
				default:
					UE_LOGF(LogLocalFileReplay, Warning, "ReadReplayInfo: Unhandled file chunk type: %d", (uint32)ChunkType);
					break;
				}

				if (Archive.IsError())
				{
					UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Archive error after parsing chunk");
					return false;
				}

				Archive.Seek(Chunk.DataOffset + Chunk.SizeInBytes);
			}
		}

		if (SerializationInfo.GetLocalFileReplayVersion() < FLocalFileReplayCustomVersion::StreamChunkTimes)
		{
			for(int i=0; i < Info.DataChunks.Num(); ++i)
			{
				const int32 CheckpointStartIdx = i - 1;

				if (Info.Checkpoints.IsValidIndex(CheckpointStartIdx))
				{
					Info.DataChunks[i].Time1 = Info.Checkpoints[CheckpointStartIdx].Time1;
				}
				else
				{
					Info.DataChunks[i].Time1 = 0;
				}					

				if (Info.Checkpoints.IsValidIndex(i))
				{
					Info.DataChunks[i].Time2 = Info.Checkpoints[i].Time1;
				}
				else
				{
					Info.DataChunks[i].Time2 = Info.LengthInMS;
				}
			}
		}

		// check for overlapping data chunk times
		for(const FLocalFileReplayDataInfo& DataInfo : Info.DataChunks)
		{
			TInterval<uint32> Range1(DataInfo.Time1, DataInfo.Time2);

			for(const FLocalFileReplayDataInfo& DataInfoCompare : Info.DataChunks)
			{
				if (DataInfo.ChunkIndex != DataInfoCompare.ChunkIndex)
				{
					const TInterval<uint32> Range2(DataInfoCompare.Time1, DataInfoCompare.Time2);
					const TInterval<uint32> Overlap = Intersect(Range1, Range2);

					if (Overlap.IsValid() && Overlap.Size() > 0)
					{
						UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Found overlapping data chunks");
						Archive.SetError();
						return false;
					}
				}
			}
		}

		// checkpoints should be unique
		TSet<FString> CheckpointIds;

		for (const FLocalFileEventInfo& Checkpoint : Info.Checkpoints)
		{
			if (CheckpointIds.Contains(Checkpoint.Id))
			{
				UE_LOGF(LogLocalFileReplay, Error, "ReadReplayInfo: Found duplicate checkpoint id: %ls", *Checkpoint.Id);
				Archive.SetError();
				return false;
			}

			CheckpointIds.Add(Checkpoint.Id);
		}

		Info.bIsValid = EnumHasAnyFlags(Flags, EReadReplayInfoFlags::SkipHeaderChunkTest) || Info.Chunks.IsValidIndex(Info.HeaderChunkIndex);

		FArchiveCountMem CountMemAr(nullptr);
		Info.CountBytes(CountMemAr);
		const int64 InfoSize = sizeof(FLocalFileReplayInfo) + CountMemAr.GetNum();

		UE_LOGF(LogLocalFileReplay, Verbose, "ReadReplayInfo: IsValid: %ls MemSize: %lld bytes", *LexToString(Info.bIsValid), InfoSize);

		return Info.bIsValid && !Archive.IsError();
	}

	return false;
}

bool FLocalFileNetworkReplayStreamer::WriteReplayInfo(FArchive& Archive, const FLocalFileReplayInfo& InReplayInfo)
{
	FLocalFileSerializationInfo DefaultSerializationInfo;
	return WriteReplayInfo(Archive, InReplayInfo, DefaultSerializationInfo);
}

bool FLocalFileNetworkReplayStreamer::AllowEncryptedWrite() const
{
	bool bAllowWrite = SupportsEncryption();

#if !UE_BUILD_SHIPPING
	bAllowWrite = bAllowWrite && (UE::Net::LocalFileReplay::CVarAllowEncryptedRecording.GetValueOnAnyThread() != 0);

	UE_LOGF(LogLocalFileReplay, VeryVerbose, "FLocalFileNetworkReplayStreamer::AllowEncryptedWrite: %ls", *LexToString(bAllowWrite));
#endif

	return bAllowWrite;
}

bool FLocalFileNetworkReplayStreamer::WriteReplayInfo(FArchive& Archive, const FLocalFileReplayInfo& InReplayInfo, FLocalFileSerializationInfo& SerializationInfo)
{
	if (SerializationInfo.GetLocalFileReplayVersion() < FLocalFileReplayCustomVersion::FixedSizeFriendlyName)
	{
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkRepalyStreamer::WriteReplayInfo: Unable to safely rewrite old replay info");
		return false;
	}

	Archive.Seek(0);

	uint32 MagicNumber = FLocalFileNetworkReplayStreamer::FileMagic;
	Archive << MagicNumber;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Archive << SerializationInfo.FileVersion;

	if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::CustomVersions)
	{
		SerializationInfo.FileCustomVersions.Serialize(Archive);

		Archive.SetCustomVersions(SerializationInfo.FileCustomVersions);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Archive << const_cast<int32&>(InReplayInfo.LengthInMS);
	Archive << const_cast<uint32&>(InReplayInfo.NetworkVersion);
	Archive << const_cast<uint32&>(InReplayInfo.Changelist);

	FString FixedSizeName;
	FixupFriendlyNameLength(InReplayInfo.FriendlyName, FixedSizeName);

	if (SerializationInfo.GetLocalFileReplayVersion() < FLocalFileReplayCustomVersion::FriendlyNameCharEncoding)
	{
		// if the new name contains non-ANSI characters and the old does not, serializing would corrupt the file
		if (!FCString::IsPureAnsi(*FixedSizeName) && FCString::IsPureAnsi(*SerializationInfo.FileFriendlyName))
		{
			UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkRepalyStreamer::WriteReplayInfo: Forcing friendly name to ANSI to avoid corrupting file");
			
			FString ConvertedName = TCHAR_TO_ANSI(*FixedSizeName);
			Archive << ConvertedName;
		}
		// otherwise if the old name has non-ANSI character, force unicode
		else if (!FCString::IsPureAnsi(*SerializationInfo.FileFriendlyName))
		{
			bool bForceUnicode = Archive.IsForcingUnicode();
			Archive.SetForceUnicode(true);
			Archive << FixedSizeName;
			Archive.SetForceUnicode(bForceUnicode);
		}
		else // both are ANSI, just write the string
		{
			Archive << FixedSizeName;
		}
	}
	else
	{
		// force unicode so the size will actually be fixed
		bool bForceUnicode = Archive.IsForcingUnicode();
		Archive.SetForceUnicode(true);
		Archive << FixedSizeName;
		Archive.SetForceUnicode(bForceUnicode);
	}

	uint32 IsLive = InReplayInfo.bIsLive ? 1 : 0;
	Archive << IsLive;

	// It's possible we're updating an older replay (e.g., for a rename)
	// Therefore, we can't write out any data that the replay wouldn't have had.
	if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::RecordingTimestamp)
	{
		Archive << const_cast<FDateTime&>(InReplayInfo.Timestamp);
	}

	if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::CompressionSupport)
	{
		uint32 Compressed = SupportsCompression() ? 1 : 0;
		Archive << Compressed;
	}

	if (SerializationInfo.GetLocalFileReplayVersion() >= FLocalFileReplayCustomVersion::EncryptionSupport)
	{
		uint32 Encrypted = AllowEncryptedWrite() ? 1 : 0;
		Archive << Encrypted;

		TArray<uint8> KeyToWrite;

		if (InReplayInfo.bIsLive)
		{
			KeyToWrite.AddZeroed(InReplayInfo.EncryptionKey.Num());
		}
		else
		{
			KeyToWrite = InReplayInfo.EncryptionKey;
		}

		Archive << KeyToWrite;
	}

	return !Archive.IsError();
}

void FLocalFileNetworkReplayStreamer::FixupFriendlyNameLength(const FString& UnfixedName, FString& FixedName) const
{
	const uint32 DesiredLength = GetMaxFriendlyNameSize();
	const uint32 NameLen = UnfixedName.Len();
	if (NameLen < DesiredLength)
	{
		FixedName = UnfixedName.RightPad(DesiredLength);
	}
	else
	{
		FixedName = UnfixedName.Left(DesiredLength);
	}
}

void FLocalFileNetworkReplayStreamer::StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate)
{
	FStartStreamingResult Result;
	Result.bRecording = Params.bRecord;

	if (IsStreaming())
	{
		UE_LOGF( LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::StartStreaming. IsStreaming == true." );
		Delegate.ExecuteIfBound(Result);
		return;
	}

	if (IsFileRequestInProgress())
	{
		UE_LOGF( LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::StartStreaming. IsFileRequestInProgress == true." );
		Delegate.ExecuteIfBound(Result);
		return;
	}

	if (Params.CustomName.IsEmpty() && !Params.bRecord)
	{
		// Can't play a replay if the user didn't provide a name!
		Result.Result = EStreamingOperationResult::ReplayNotFound;
		Delegate.ExecuteIfBound(Result);
		return;
	}

	// Setup the archives
	StreamAr.SetIsLoading(!Params.bRecord);
	StreamAr.SetIsSaving(!StreamAr.IsLoading());
	StreamAr.bAtEndOfReplay = false;

	HeaderAr.SetIsLoading(StreamAr.IsLoading());
	HeaderAr.SetIsSaving(StreamAr.IsSaving());

	CheckpointAr.SetIsLoading(StreamAr.IsLoading());
	CheckpointAr.SetIsSaving(StreamAr.IsSaving());

	CurrentReplayInfo.LengthInMS = 0;

	StreamTimeRange = TInterval<uint32>(0, 0);

	StreamDataOffset = 0;
	StreamChunkIndex = 0;

	if (!Params.bRecord)
	{
		// We are playing
		StreamerState = EReplayStreamerState::Playback;

		// Add the request to start loading
		AddDelegateFileRequestToQueue<FStartStreamingResult>(EQueuedLocalFileRequestType::StartPlayback, 
			[this, Params](TLocalFileRequestCommonData<FStartStreamingResult>& RequestData)
			{
				CurrentStreamName = Params.CustomName;

				const FString FullDemoFilename = GetDemoFullFilename(Params.CustomName);

				RequestData.DelegateResult.bRecording = false;
				
				if (!FPaths::FileExists(FullDemoFilename))
				{
					RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
				}
				else
				{
					// Load metadata if it exists
					ReadReplayInfo(CurrentStreamName, TaskReplayInfo);
				}
			},
			[this, Delegate](TLocalFileRequestCommonData<FStartStreamingResult>& RequestData)
			{
				if (RequestData.DelegateResult.Result == EStreamingOperationResult::ReplayNotFound)
				{
					Delegate.ExecuteIfBound(RequestData.DelegateResult);
				}
				else
				{
					UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::FullUpdate);

					if (!CurrentReplayInfo.bIsValid)
					{
						RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayCorrupt;

						Delegate.ExecuteIfBound(RequestData.DelegateResult);
					}
					else
					{
						DownloadHeader(FDownloadHeaderCallback());
						AddDelegateFileRequestToQueue<FStartStreamingCallback, FStartStreamingResult>(EQueuedLocalFileRequestType::StartPlayback, FStartStreamingCallback(),
							[this, Delegate](TLocalFileRequestCommonData<FStartStreamingResult>& PlaybackRequestData)
							{
								PostHeaderDownloaded([this, Delegate]()
									{
										if (StreamerState == EReplayStreamerState::Idle || bStopStreamingCalled)
										{
											// we were cancelled during operation.
											return;
										}
										AddDelegateFileRequestToQueue<FStartStreamingCallback, FStartStreamingResult>(EQueuedLocalFileRequestType::StartPlayback, Delegate,
											[this](TLocalFileRequestCommonData<FStartStreamingResult>& PlaybackRequestData)
											{
												PlaybackRequestData.DelegateResult.bRecording = false;

												if (CurrentReplayInfo.bIsValid)
												{
													PlaybackRequestData.DelegateResult.Result = EStreamingOperationResult::Success;
												}
											});
									}
								);
							});
					}
				}
			});
	}
	else
	{
		// We are recording
		StreamerState = EReplayStreamerState::Recording;

		uint64 TotalDiskFreeSpace = 0;
		if (GetDemoFreeStorageSpace(TotalDiskFreeSpace, GetDemoPath()))
		{
			UE_LOGF(LogLocalFileReplay, Log, "Writing replay to '%ls' with %.2fMB free", *GetDemoPath(), (double)TotalDiskFreeSpace / 1024 / 1024);
		}

		CurrentReplayInfo.EncryptionKey.Reset();

		// generate key now in case any other events are queued during the initial write
		if (AllowEncryptedWrite())
		{
			GenerateEncryptionKey(CurrentReplayInfo.EncryptionKey);
		}

		// LocalFileAr should be null if IsStreaming() returned false earlier */
		check(LocalFileAr == nullptr);

		AddDelegateFileRequestToQueue<FStartStreamingResult>(EQueuedLocalFileRequestType::StartRecording,
			[this, Params, EncryptionKey=CurrentReplayInfo.EncryptionKey](TLocalFileRequestCommonData<FStartStreamingResult>& RequestData)
			{
				SCOPE_CYCLE_COUNTER(STAT_LocalReplay_StartRecording);

				FString FinalDemoName = Params.CustomName;

				if (Params.CustomName.IsEmpty())
				{
					// If we're recording and the caller didn't provide a name, generate one automatically
					FinalDemoName = GetAutomaticDemoName();
				}

				CurrentStreamName = FinalDemoName;

				UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::StartStreaming. Filename: %ls.", *FinalDemoName);

				const FString FullDemoFilename = GetDemoFullFilename(FinalDemoName);

				// only record to valid replay file names
				if ((FinalDemoName.IsEmpty() || !FullDemoFilename.EndsWith(FNetworkReplayStreaming::GetReplayFileExtension())))
				{
					UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::StartStreaming. Invalid replay file name for recording: %ls.", *FullDemoFilename);
					RequestData.AsyncError = ELocalFileReplayResult::InvalidName;
					return;
				}

				// Delete any existing demo with this name if it exists
				if (IFileManager::Get().FileExists(*FullDemoFilename))
				{
					FLocalFileReplayInfo ExistingInfo;
					if (ReadReplayInfo(FinalDemoName, ExistingInfo))
					{
						if (ExistingInfo.bIsLive)
						{
							UE_LOGF(LogLocalFileReplay, Warning, "StartStreaming is overwriting an existing live replay file.");
						}
					}
					IFileManager::Get().Delete(*FullDemoFilename);
				}

				LocalFileAr = CreateLocalFileWriter(FullDemoFilename);
				if (!LocalFileAr.IsValid())
				{
					UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::StartStreaming. Invalid LocalFileAr. You may need to enable localReplay.UsePlayerAccountIdPostfix.");
					RequestData.AsyncError = ELocalFileReplayResult::FileWriter;
					TaskReplayInfo.bIsValid = false;
					return;
				}

				RequestData.DelegateResult.bRecording = true;

				TaskReplayInfo.NetworkVersion = Params.ReplayVersion.NetworkVersion;
				TaskReplayInfo.Changelist = Params.ReplayVersion.Changelist;
				TaskReplayInfo.FriendlyName = Params.FriendlyName;
				TaskReplayInfo.bIsLive = true;
				TaskReplayInfo.Timestamp = FDateTime::Now();
				TaskReplayInfo.EncryptionKey = EncryptionKey;

				TaskReplayInfo.bIsValid = WriteReplayInfo(*LocalFileAr.Get(), TaskReplayInfo);

				if (!TaskReplayInfo.bIsValid)
				{
					UE_LOGF(LogLocalFileReplay, Warning, "StartStreaming was unable to write to the replay file: %ls", *FullDemoFilename);
					RequestData.AsyncError = ELocalFileReplayResult::FileWriter;
				}
				else
				{
					RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
				}
			},
			[this, Delegate](TLocalFileRequestCommonData<FStartStreamingResult>& RequestData)
			{
				if (RequestData.AsyncError == ELocalFileReplayResult::Success)
				{
					UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::FullUpdate);
				}
				else
				{
					UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::StartStreaming. AsyncError: %d.", static_cast<uint32>(RequestData.AsyncError));
				}

				Delegate.ExecuteIfBound(RequestData.DelegateResult);
			});

		RefreshHeader();
	}
}

void FLocalFileNetworkReplayStreamer::CancelStreamingRequests()
{
	// Cancel any active request
	if (ActiveRequest.IsValid())
	{
		ActiveRequest->CancelRequest();
		ActiveRequest = nullptr;
	}

	// Empty the request queue
	QueuedRequests.Empty();

	StreamerState = EReplayStreamerState::Idle;
	bStopStreamingCalled = false;
}

void FLocalFileNetworkReplayStreamer::SetLastError(FLocalFileReplayResult&& Result)
{
	SetExtendedError(MoveTemp(Result));

	CancelStreamingRequests();
}

void FLocalFileNetworkReplayStreamer::StopStreaming()
{
	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::StartPlayback) || IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::StartRecording))
	{
		UE_LOGF( LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::StopStreaming. Called while existing StartStreaming request wasn't finished" );
		CancelStreamingRequests();
		check( !IsStreaming() );
		return;
	}

	if (!IsStreaming())
	{
		UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::StopStreaming. Not currently streaming.");
		check( bStopStreamingCalled == false );
		return;
	}

	if (bStopStreamingCalled)
	{
		UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::StopStreaming. Already called");
		return;
	}

	bStopStreamingCalled = true;

	if (StreamerState == EReplayStreamerState::Recording)
	{
		// Flush any final pending stream
		int32 TotalLengthInMS = CurrentReplayInfo.LengthInMS;

		FlushStream(TotalLengthInMS);

		AddGenericRequestToQueue<ELocalFileReplayResult>(EQueuedLocalFileRequestType::StopRecording,
			[this, TotalLengthInMS](ELocalFileReplayResult& ReplayResult)
			{
				if (LocalFileAr.IsValid())
				{
					// Set the final values of these header properties
					TaskReplayInfo.bIsLive = false;
					TaskReplayInfo.LengthInMS = TotalLengthInMS;
					TaskReplayInfo.EncryptionKey = CurrentReplayInfo.EncryptionKey;

					WriteReplayInfo(*LocalFileAr.Get(), TaskReplayInfo);
				}
				LocalFileAr = nullptr;
			},
			[this](ELocalFileReplayResult& ReplayResult)
			{
				UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::FullUpdate);
			});
	}

	// Finally, add the stop streaming request, which should put things in the right state after the above requests are done
	AddSimpleRequestToQueue(EQueuedLocalFileRequestType::StopStreaming,
		[]()
		{
			UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::StopStreaming");
		},
		[this]()
		{
			bStopStreamingCalled = false;
			StreamAr.SetIsLoading(false);
			StreamAr.SetIsSaving(false);
			StreamAr.Reset();
			StreamDataOffset = 0;
			StreamChunkIndex = 0;
			StreamerState = EReplayStreamerState::Idle;

			FString StreamName = MoveTemp(CurrentStreamName);
			const FString FullFileName = GetDemoFullFilename(StreamName);

			CurrentStreamName.Empty();

			OnReplayFinishedWriting.Broadcast(StreamName, FullFileName);
		});
}

FArchive* FLocalFileNetworkReplayStreamer::GetHeaderArchive()
{
	return &HeaderAr;
}

FArchive* FLocalFileNetworkReplayStreamer::GetStreamingArchive()
{
	return &StreamAr;
}

void FLocalFileNetworkReplayStreamer::UpdateTotalDemoTime(uint32 TimeInMS)
{
	check(StreamerState == EReplayStreamerState::Recording);

	CurrentReplayInfo.LengthInMS = TimeInMS;
}

bool FLocalFileNetworkReplayStreamer::IsDataAvailable() const
{
	if (HasError())
	{
		return false;
	}

	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::ReadingCheckpoint))
	{
		return false;
	}

	if (HighPriorityEndTime > 0)
	{
		// If we are waiting for a high priority portion of the stream, pretend like we don't have any data so that game code waits for the entire portion
		// of the high priority stream to load.
		// We do this because we assume the game wants to race through this high priority portion of the stream in a single frame
		return false;
	}

	// If we are loading, and we have more data
	if (StreamAr.IsLoading() && StreamAr.Tell() < StreamAr.TotalSize() && CurrentReplayInfo.DataChunks.Num() > 0)
	{
		return true;
	}

	return false;
}

bool FLocalFileNetworkReplayStreamer::IsLive() const
{
	return CurrentReplayInfo.bIsLive;
}

bool FLocalFileNetworkReplayStreamer::IsNamedStreamLive(const FString& StreamName) const
{
	check(!IsInGameThread());

	FLocalFileReplayInfo Info;
	return ReadReplayInfo(StreamName, Info) && Info.bIsLive;
}

void FLocalFileNetworkReplayStreamer::DeleteFinishedStream(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate)
{
	DeleteFinishedStream_Internal(StreamName, UserIndex, Delegate);
}

void FLocalFileNetworkReplayStreamer::DeleteFinishedStream(const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate)
{
	DeleteFinishedStream_Internal(StreamName, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::DeleteFinishedStream_Internal(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FDeleteFinishedStreamCallback, FDeleteFinishedStreamResult>(EQueuedLocalFileRequestType::DeletingFinishedStream, Delegate,
		[this, StreamName](TLocalFileRequestCommonData<FDeleteFinishedStreamResult>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_Delete);

			const bool bIsLive = IsNamedStreamLive(StreamName);

			if (UE::Net::LocalFileReplay::CVarAllowLiveStreamDelete.GetValueOnAnyThread() || !bIsLive)
			{
				UE_CLOGF(bIsLive, LogLocalFileReplay, Warning, "Deleting network replay stream %ls that is currently live!", *StreamName);

				const FString FullDemoFilename = GetDemoFullFilename(StreamName);
			
				if (!FPaths::FileExists(FullDemoFilename))
				{
					RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
				}
				else if (IFileManager::Get().Delete(*FullDemoFilename))
				{
					RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
				}
			}
			else
			{
				UE_LOGF(LogLocalFileReplay, Warning, "Can't delete network replay stream %ls because it is live!", *StreamName);
			}
		});
}

void FLocalFileNetworkReplayStreamer::EnumerateStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FEnumerateStreamsCallback, FEnumerateStreamsResult>(EQueuedLocalFileRequestType::EnumeratingStreams, Delegate,
		[this, ReplayVersion](TLocalFileRequestCommonData<FEnumerateStreamsResult>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_Enumerate);

			const FString WildCardPath = GetDemoPath() + TEXT("*") + FNetworkReplayStreaming::GetReplayFileExtension();

			TArray<FString> ReplayFileNames;
			IFileManager::Get().FindFiles(ReplayFileNames, *WildCardPath, true, false);

			for (const FString& ReplayFileName : ReplayFileNames)
			{
				// Read stored info for this replay
				FLocalFileReplayInfo StoredReplayInfo;
				if (!ReadReplayInfo(FPaths::GetBaseFilename(ReplayFileName), StoredReplayInfo))
				{
					continue;
				}

				// Check version. NetworkVersion and changelist of 0 will ignore version check.
				const bool bNetworkVersionMatches = ReplayVersion.NetworkVersion == StoredReplayInfo.NetworkVersion;
				const bool bChangelistMatches = ReplayVersion.Changelist == StoredReplayInfo.Changelist;

				const bool bNetworkVersionPasses = ReplayVersion.NetworkVersion == 0 || bNetworkVersionMatches;
				const bool bChangelistPasses = ReplayVersion.Changelist == 0 || bChangelistMatches;

				if (bNetworkVersionPasses && bChangelistPasses)
				{
					FNetworkReplayStreamInfo Info;

					Info.Name = FPaths::GetBaseFilename(ReplayFileName);
					Info.bIsLive = StoredReplayInfo.bIsLive;
					Info.Changelist = StoredReplayInfo.Changelist; 
					Info.LengthInMS = StoredReplayInfo.LengthInMS;
					Info.FriendlyName = StoredReplayInfo.FriendlyName;
					Info.SizeInBytes = StoredReplayInfo.TotalDataSizeInBytes;
					Info.Timestamp = StoredReplayInfo.Timestamp;

					// If we don't have a valid timestamp, assume it's the file's timestamp.
					if (Info.Timestamp == FDateTime::MinValue())
					{
						Info.Timestamp = IFileManager::Get().GetTimeStamp(*GetDemoFullFilename(Info.Name));
					}

					RequestData.DelegateResult.FoundStreams.Add(Info);
				}
			}

			RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
		});
}

void FLocalFileNetworkReplayStreamer::EnumerateRecentStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate)
{
	UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::EnumerateRecentStreamsEnumerateRecentStreams is currently unsupported.");
	FEnumerateStreamsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.ExecuteIfBound(Result);
}

void FLocalFileNetworkReplayStreamer::AddUserToReplay(const FString& UserString)
{
	UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::AddUserToReplay is currently unsupported.");
}

void FLocalFileNetworkReplayStreamer::AddEvent(const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	if (StreamerState != EReplayStreamerState::Recording)
	{
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::AddEvent. Not recording.");
		return;
	}

	AddOrUpdateEvent(TEXT(""), TimeInMS, Group, Meta, Data);
}

void FLocalFileNetworkReplayStreamer::AddOrUpdateEvent(const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data)
{
	if (StreamerState != EReplayStreamerState::Recording)
	{
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::AddOrUpdateEvent. Not recording.");
		return;
	}

	UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::AddOrUpdateEvent. Size: %i", Data.Num());

	AddGenericRequestToQueue<ELocalFileReplayResult>(EQueuedLocalFileRequestType::UpdatingEvent,
		[this, Name, Group, TimeInMS, Meta, Data](ELocalFileReplayResult& ReplayResult)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_FlushEvent);

			FString EventName = Name;

			// if name is empty, assign one
			if (EventName.IsEmpty())
			{
				EventName = FGuid::NewGuid().ToString(EGuidFormats::Digits);
			}

			// prefix with stream name to be consistent with http streamer
			EventName = CurrentStreamName + TEXT("_") + EventName;

			if (LocalFileAr.IsValid())
			{
				int32 EventIndex = INDEX_NONE;

				// see if this event already exists
				for (int32 i=0; i < TaskReplayInfo.Events.Num(); ++i)
				{
					if (TaskReplayInfo.Events[i].Id == EventName)
					{
						EventIndex = i;
						break;
					}
				}

				TArray<uint8> EncryptedData;

				if (AllowEncryptedWrite())
				{
					SCOPE_CYCLE_COUNTER(STAT_LocalReplay_EncryptTime);

					if (!EncryptBuffer(Data, EncryptedData, TaskReplayInfo.EncryptionKey))
					{
						UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::AddOrUpdateEvent - EncryptBuffer failed");
						ReplayResult = ELocalFileReplayResult::EncryptBuffer;
						return;
					}
				}
				else
				{
					EncryptedData = Data;
				}

				// serialize event to temporary location
				FArrayWriter Writer;

				ELocalFileChunkType ChunkType = ELocalFileChunkType::Event;
				Writer << ChunkType;

				const int64 SavedPos = Writer.Tell();

				int32 PlaceholderSize = 0;
				Writer << PlaceholderSize;

				const int64 MetadataPos = Writer.Tell();

				FString TempId = EventName;
				Writer << TempId;

				FString GroupValue = Group;
				Writer << GroupValue;

				FString MetaValue = Meta;
				Writer << MetaValue;

				uint32 Time1 = TimeInMS;
				Writer << Time1;

				uint32 Time2 = TimeInMS;
				Writer << Time2;

				int32 EventSize = EncryptedData.Num();
				Writer << EventSize;

				const int64 InternalDataOffset = Writer.Tell();

				Writer.Serialize((void*)EncryptedData.GetData(), EncryptedData.Num());

				int32 ChunkSize = IntCastChecked<int32>(Writer.Tell() - MetadataPos);

				bool bNewChunk = true;

				if (EventIndex == INDEX_NONE)
				{
					// append new event chunk
					LocalFileAr->Seek(LocalFileAr->TotalSize());
				}
				else 
				{
					if (ChunkSize > TaskReplayInfo.Chunks[TaskReplayInfo.Events[EventIndex].ChunkIndex].SizeInBytes)
					{
						LocalFileAr->Seek(TaskReplayInfo.Chunks[TaskReplayInfo.Events[EventIndex].ChunkIndex].TypeOffset);

						// clear chunk type so it will be skipped later
						ChunkType = ELocalFileChunkType::Unknown;
						*LocalFileAr << ChunkType;

						TaskReplayInfo.Chunks[TaskReplayInfo.Events[EventIndex].ChunkIndex].ChunkType = ELocalFileChunkType::Unknown;

						LocalFileAr->Seek(LocalFileAr->TotalSize());
					}
					else
					{
						bNewChunk = false;

						// reuse existing chunk
						LocalFileAr->Seek(TaskReplayInfo.Chunks[TaskReplayInfo.Events[EventIndex].ChunkIndex].TypeOffset);

						// maintain the original chunk size to avoid corrupting the file
						ChunkSize = TaskReplayInfo.Chunks[TaskReplayInfo.Events[EventIndex].ChunkIndex].SizeInBytes;
					}
				}

				Writer.Seek(SavedPos);
				Writer << ChunkSize;

				const int64 TypeOffset = LocalFileAr->Tell();

				if (bNewChunk)
				{
					FLocalFileChunkInfo& ChunkInfo = TaskReplayInfo.Chunks.AddDefaulted_GetRef();
					ChunkInfo.ChunkType = ChunkType;
					ChunkInfo.TypeOffset = TypeOffset;
					ChunkInfo.DataOffset = TypeOffset + MetadataPos;
					ChunkInfo.SizeInBytes = ChunkSize;

					FLocalFileEventInfo& NewEventInfo = TaskReplayInfo.Events.AddDefaulted_GetRef();
					NewEventInfo.ChunkIndex = TaskReplayInfo.Chunks.Num() - 1;
					NewEventInfo.Id = MoveTemp(TempId);

					EventIndex = TaskReplayInfo.Events.Num() - 1;
				}

				FLocalFileEventInfo& EventInfo = TaskReplayInfo.Events[EventIndex];
				EventInfo.Group = MoveTemp(GroupValue);
				EventInfo.Metadata = MoveTemp(MetaValue);
				EventInfo.Time1 = Time1;
				EventInfo.Time2 = Time2;
				EventInfo.SizeInBytes = EventSize;
				EventInfo.EventDataOffset = TypeOffset + InternalDataOffset;

				LocalFileAr->Serialize(Writer.GetData(), Writer.TotalSize());
			}
			else
			{
				ReplayResult = ELocalFileReplayResult::FileWriter;
			}
		},
		[this](ELocalFileReplayResult& ReplayResult)
		{
			if (ReplayResult == ELocalFileReplayResult::Success)
			{
				UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::None);
			}
			else
			{
				SetLastError(ReplayResult);
			}
		});
}

void FLocalFileNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents_Internal(ReplayName, Group, UserIndex, Delegate);
}

void FLocalFileNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents_Internal(ReplayName, Group, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents_Internal(FString(), Group, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::EnumerateEvents_Internal(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FEnumerateEventsCallback, FEnumerateEventsResult>(EQueuedLocalFileRequestType::EnumeratingEvents, Delegate,
		[this, ReplayName, Group](TLocalFileRequestCommonData<FEnumerateEventsResult>& RequestData)
		{
			FString FileName = ReplayName;
			if (FileName.IsEmpty())
			{
				FileName = CurrentStreamName;
			}

			if (!FPaths::FileExists(GetDemoFullFilename(FileName)))
			{
				RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
			}
			else
			{
				// Read stored info for this replay
				FLocalFileReplayInfo StoredReplayInfo;
				if (ReadReplayInfo(FileName, StoredReplayInfo))
				{
					for (const FLocalFileEventInfo& EventInfo : StoredReplayInfo.Events)
					{
						if (Group.IsEmpty() || (EventInfo.Group == Group))
						{
							int Idx = RequestData.DelegateResult.ReplayEventList.ReplayEvents.AddDefaulted();

							FReplayEventListItem& Event = RequestData.DelegateResult.ReplayEventList.ReplayEvents[Idx];
							Event.ID = EventInfo.Id;
							Event.Group = EventInfo.Group;
							Event.Metadata = EventInfo.Metadata;
							Event.Time1 = EventInfo.Time1;
							Event.Time2 = EventInfo.Time2;
						}
					}

					RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
				}
			}
		});
}

void FLocalFileNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	RequestEventData_Internal(ReplayName, EventID, UserIndex, Delegate);
}

void FLocalFileNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	RequestEventData_Internal(ReplayName, EventID, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	RequestEventData_Internal(FString(), EventID, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::RequestEventData_Internal(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& RequestEventDataComplete)
{
	AddDelegateFileRequestToQueue<FRequestEventDataCallback, FRequestEventDataResult>(EQueuedLocalFileRequestType::RequestingEvent, RequestEventDataComplete,
		[this, ReplayName, EventID](TLocalFileRequestCommonData<FRequestEventDataResult>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_ReadEvent);

			FString FileName = ReplayName;
			if (FileName.IsEmpty())
			{
				// Assume current stream
				FileName = CurrentStreamName;

				// But look for name prefix, http streamer expects to pull details from arbitrary streams
				int32 Idx = INDEX_NONE;
				if (EventID.FindChar(TEXT('_'), Idx))
				{
					FileName = EventID.Left(Idx);
				}
			}
			const FString FullDemoFilename = GetDemoFullFilename(FileName);
			if (!FPaths::FileExists(FullDemoFilename))
			{
				RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
			}
			else
			{
				// Read stored info for this replay
				FLocalFileReplayInfo StoredReplayInfo;
				if (ReadReplayInfo(FileName, StoredReplayInfo))
				{
					TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullDemoFilename);
					if (LocalFileReader.IsValid())
					{
						bool bEventFound = false;

						for (const FLocalFileEventInfo& EventInfo : StoredReplayInfo.Events)
						{
							if (EventInfo.Id == EventID)
							{
								bEventFound = true;

								LocalFileReader->Seek(EventInfo.EventDataOffset);

								TArray<uint8> EventData;

								EventData.AddUninitialized(EventInfo.SizeInBytes);
								LocalFileReader->Serialize(EventData.GetData(), EventData.Num());

								if (StoredReplayInfo.bEncrypted)
								{
									if (SupportsEncryption())
									{
										SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecryptTime);

										TArray<uint8> PlaintextData;
										
										if (!DecryptBuffer(EventData, PlaintextData, StoredReplayInfo.EncryptionKey))
										{
											UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. DecryptBuffer failed.");
											RequestData.DelegateResult.Result = EStreamingOperationResult::DecryptFailure;
											break;
										}

										EventData = MoveTemp(PlaintextData);
									}
									else
									{
										UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. Encrypted event but streamer does not support encryption.");
										RequestData.DelegateResult.Result = EStreamingOperationResult::Unsupported;
										break;
									}
								}

								RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
								RequestData.DelegateResult.ReplayEventListItem = MoveTemp(EventData);

								break;
							}
						}

						LocalFileReader = nullptr;

						// we didn't find the event
						if (!bEventFound)
						{
							RequestData.DelegateResult.Result = EStreamingOperationResult::EventNotFound;
						}
					}
					else
					{
						UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. Unable to read replay file: %ls", *FullDemoFilename);
						RequestData.DelegateResult.Result = EStreamingOperationResult::Unspecified;
					}
				}
				else
				{
					UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. Failed to read the replay info: %ls", *FileName);
					RequestData.DelegateResult.Result = EStreamingOperationResult::Unspecified;
				}
			}
		});
}

void FLocalFileNetworkReplayStreamer::RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(FString(), Group, Delegate);
}

void FLocalFileNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(ReplayName, Group, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FRequestEventGroupDataCallback, FRequestEventGroupDataResult>(EQueuedLocalFileRequestType::RequestingEvent, Delegate,
		[this, ReplayName, Group](TLocalFileRequestCommonData<FRequestEventGroupDataResult>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_ReadEvent);

			FString FileName = ReplayName;
			if (FileName.IsEmpty())
			{
				FileName = CurrentStreamName;
			}

			const FString FullDemoFilename = GetDemoFullFilename(FileName);
			if (!FPaths::FileExists(FullDemoFilename))
			{
				RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
			}
			else
			{
				// Read stored info for this replay
				FLocalFileReplayInfo StoredReplayInfo;
				if (ReadReplayInfo(FileName, StoredReplayInfo))
				{
					TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullDemoFilename);
					if (LocalFileReader.IsValid())
					{
						bool bGroupFound = false;

						for (const FLocalFileEventInfo& EventInfo : StoredReplayInfo.Events)
						{
							if (EventInfo.Group == Group)
							{
								bGroupFound = true;

								LocalFileReader->Seek(EventInfo.EventDataOffset);

								RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
								
								FReplayEventListItem& EventItem = RequestData.DelegateResult.ReplayEventListItems.AddDefaulted_GetRef();
								EventItem.ID = EventInfo.Id;
								EventItem.Group = EventInfo.Group;
								EventItem.Metadata = EventInfo.Metadata;
								EventItem.Time1 = EventInfo.Time1;
								EventItem.Time2 = EventInfo.Time2;

								TArray<uint8> EventData;

								EventData.AddUninitialized(EventInfo.SizeInBytes);
								LocalFileReader->Serialize(EventData.GetData(), EventData.Num());

								if (StoredReplayInfo.bEncrypted)
								{
									if (SupportsEncryption())
									{
										SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecryptTime);

										TArray<uint8> PlaintextData;
										
										if (!DecryptBuffer(EventData, PlaintextData, StoredReplayInfo.EncryptionKey))
										{
											UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. DecryptBuffer failed.");
											RequestData.DelegateResult.Result = EStreamingOperationResult::DecryptFailure;
											break;
										}

										EventData = MoveTemp(PlaintextData);
									}
									else
									{
										UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. Encrypted event but streamer does not support encryption.");
										RequestData.DelegateResult.Result = EStreamingOperationResult::Unsupported;
										break;
									}
								}

								FRequestEventDataResult& EventDataResult = RequestData.DelegateResult.ReplayEventListResults.AddDefaulted_GetRef();
								EventDataResult.Result = EStreamingOperationResult::Success;
								EventDataResult.ReplayEventListItem = MoveTemp(EventData);
							}
						}

						// we didn't find the group
						if (!bGroupFound)
						{
							RequestData.DelegateResult.Result = EStreamingOperationResult::EventNotFound;
						}

						LocalFileReader = nullptr;
					}
					else
					{
						UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. Failed to read the replay file: %ls", *FullDemoFilename);
						RequestData.DelegateResult.Result = EStreamingOperationResult::Unspecified;
					}
				}
				else
				{
					UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::RequestEventData_Internal. Failed to read the replay info: %ls", *FileName);
					RequestData.DelegateResult.Result = EStreamingOperationResult::Unspecified;
				}
			}
		});
}

void FLocalFileNetworkReplayStreamer::SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate)
{
	UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::SearchEvents is currently unsupported.");
	FSearchEventsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.ExecuteIfBound(Result);
}

void FLocalFileNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	KeepReplay_Internal(ReplayName, bKeep, UserIndex, Delegate);
}

void FLocalFileNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate)
{
	KeepReplay_Internal(ReplayName, bKeep, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::KeepReplay_Internal(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FKeepReplayCallback, FKeepReplayResult>(EQueuedLocalFileRequestType::KeepReplay, Delegate,
		[this, ReplayName](TLocalFileRequestCommonData<FKeepReplayResult>& RequestData)
		{
			// Replays are kept during streaming so there's no need to explicitly save them.
			// However, sanity check that what was passed in still exists.
			if (!FPaths::FileExists(GetDemoFullFilename(ReplayName)))
			{
				RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
			}
			else
			{
				RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
				RequestData.DelegateResult.NewReplayName = ReplayName;
			}
		});
}

void FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplayFriendlyName_Internal(ReplayName, NewFriendlyName, UserIndex, Delegate);
}

void FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate)
{
	RenameReplayFriendlyName_Internal(ReplayName, NewFriendlyName, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName_Internal(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FRenameReplayCallback, FRenameReplayResult>(EQueuedLocalFileRequestType::RenameReplayFriendlyName, Delegate,
		[this, ReplayName, NewFriendlyName](TLocalFileRequestCommonData<FRenameReplayResult>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_RenameFriendly);

			const FString& FullReplayName = GetDemoFullFilename(ReplayName);
			if (!FPaths::FileExists(FullReplayName))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName: Replay does not exist %ls", *ReplayName);
				RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
				return;
			}

			FLocalFileSerializationInfo SerializationInfo;
			FLocalFileReplayInfo TempReplayInfo;

			// Do this inside a scope, to make sure the file archive is closed before continuing.
			{
				TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullReplayName);
				if (!LocalFileReader.IsValid() || LocalFileReader->TotalSize() <= 0 || !ReadReplayInfo(*LocalFileReader.Get(), TempReplayInfo, SerializationInfo, EReadReplayInfoFlags::None))
				{
					UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName: Failed to read replay info %ls", *ReplayName);
					return;
				}

				if (SerializationInfo.GetLocalFileReplayVersion() < FLocalFileReplayCustomVersion::FixedSizeFriendlyName)
				{
					UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName: Replay too old to rename safely %ls", *ReplayName);
					return;
				}
			}

			TempReplayInfo.FriendlyName = NewFriendlyName;

			// Do this inside a scope, to make sure the file archive is closed before continuing.
			{
				// ensure existing replay is flushed and closed
				LocalFileAr = nullptr;

				TSharedPtr<FArchive> LocalFileWriter = CreateLocalFileWriter(FullReplayName);
				if (!LocalFileWriter.IsValid() || LocalFileWriter->TotalSize() <= 0 || !WriteReplayInfo(*LocalFileWriter.Get(), TempReplayInfo, SerializationInfo))
				{
					UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName: Failed to write replay info %ls", *ReplayName);
					return;
				}
			}

			RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
		});
}

void FLocalFileNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplay_Internal(ReplayName, NewName, UserIndex, Delegate);
}

void FLocalFileNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate)
{
	RenameReplay_Internal(ReplayName, NewName, INDEX_NONE, Delegate);
}

void FLocalFileNetworkReplayStreamer::RenameReplay_Internal(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	AddDelegateFileRequestToQueue<FRenameReplayCallback, FRenameReplayResult>(EQueuedLocalFileRequestType::RenameReplay, Delegate,
		[this, ReplayName, NewName](TLocalFileRequestCommonData<FRenameReplayResult>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_Rename);

			const FString& FullReplayName = GetDemoFullFilename(ReplayName);
			if (!FPaths::FileExists(FullReplayName))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplay: Replay does not exist (old %ls new %ls)", *ReplayName, *NewName);
				RequestData.DelegateResult.Result = EStreamingOperationResult::ReplayNotFound;
				return;
			}

			const FString& NewReplayName = GetDemoFullFilename(NewName);

			FString NewReplayBaseName = FPaths::GetBaseFilename(NewReplayName);
			NewReplayBaseName.RemoveFromEnd(FNetworkReplayStreaming::GetReplayFileExtension());

			// Sanity check to make sure the input name isn't changing directories.
			if (NewName != NewReplayBaseName)
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplay: Path separator characters present in replay (old %ls new %ls)", *ReplayName, *NewName);
				return;
			}

			if (!IFileManager::Get().Move(*NewReplayName, *FullReplayName, /* bReplace= */ false))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::RenameReplay: Failed to rename replay (old %ls new %ls)", *ReplayName, *NewName);
				return;
			}

			RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
		});
}

FArchive* FLocalFileNetworkReplayStreamer::GetCheckpointArchive()
{
	return &CheckpointAr;
}

void FLocalFileNetworkReplayStreamer::FlushStream(const uint32 TimeInMS)
{
	check( StreamAr.IsSaving() );

	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::WriteHeader))
	{
		// If we haven't uploaded the header, or we are not recording, we don't need to flush
		UE_LOGF( LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::FlushStream. Waiting on header upload." );
		return;
	}

	if (StreamAr.TotalSize() == 0)
	{
		// Nothing to flush
		return;
	}

	StreamTimeRange.Max = TimeInMS;

	const uint32 StreamChunkStartMS = StreamTimeRange.Min;
	const uint32 StreamChunkEndMS = StreamTimeRange.Max;

	StreamTimeRange.Min = StreamTimeRange.Max;

	const int32 TotalLengthInMS = CurrentReplayInfo.LengthInMS;

	// Save any newly streamed data to disk
	UE_LOG(LogLocalFileReplay, Verbose, TEXT("FLocalFileNetworkReplayStreamer::FlushStream. StreamChunkIndex: %i, Size: %" INT64_FMT), StreamChunkIndex, StreamAr.TotalSize());

	AddGenericRequestToQueue<ELocalFileReplayResult>(EQueuedLocalFileRequestType::WritingStream,
		[this, StreamChunkStartMS, StreamChunkEndMS, TotalLengthInMS, StreamData = MoveTemp(StreamAr.Buffer)](ELocalFileReplayResult& ReplayResult) mutable
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_FlushStream);

			if (LocalFileAr.IsValid())
			{
				LocalFileAr->Seek(LocalFileAr->TotalSize());

				int32 SizeInMemory = StreamData.Num();

				TArray<uint8> CompressedData;

				if (SupportsCompression())
				{
					SCOPE_CYCLE_COUNTER(STAT_LocalReplay_CompressTime);

					if (!CompressBuffer(StreamData, CompressedData))
					{
						UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::FlushStream - CompressBuffer failed");
						ReplayResult = ELocalFileReplayResult::CompressBuffer;
						return;
					}
				}
				else
				{
					CompressedData = MoveTemp(StreamData);
				}

				TArray<uint8> EncryptedData;

				if (AllowEncryptedWrite())
				{
					SCOPE_CYCLE_COUNTER(STAT_LocalReplay_EncryptTime);

					if (!EncryptBuffer(CompressedData, EncryptedData, TaskReplayInfo.EncryptionKey))
					{
						UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::FlushStream - EncryptBuffer failed");
						ReplayResult = ELocalFileReplayResult::EncryptBuffer;
						return;
					}
				}
				else
				{
					EncryptedData = MoveTemp(CompressedData);
				}

				// flush chunk to disk
				if (EncryptedData.Num() > 0)
				{
					const int64 TypeOffset = LocalFileAr->Tell();

					ELocalFileChunkType ChunkType = ELocalFileChunkType::ReplayData;
					*LocalFileAr << ChunkType;

					const int64 SavedPos = LocalFileAr->Tell();

					int32 PlaceholderSize = 0;
					*LocalFileAr << PlaceholderSize;

					const int64 MetadataPos = LocalFileAr->Tell();

					uint32 Time1 = StreamChunkStartMS;
					*LocalFileAr << Time1;

					uint32 Time2 = StreamChunkEndMS;
					*LocalFileAr << Time2;

					int32 SizeOnDisk = EncryptedData.Num();
					*LocalFileAr << SizeOnDisk;

					*LocalFileAr << SizeInMemory;

					const int64 ReplayDataOffset = LocalFileAr->Tell();

					LocalFileAr->Serialize((void*)EncryptedData.GetData(), EncryptedData.Num());

					int32 ChunkSize = IntCastChecked<int32>(LocalFileAr->Tell() - MetadataPos);

					LocalFileAr->Seek(SavedPos);
					*LocalFileAr << ChunkSize;

					FLocalFileChunkInfo& ChunkInfo = TaskReplayInfo.Chunks.AddDefaulted_GetRef();
					ChunkInfo.ChunkType = ChunkType;
					ChunkInfo.TypeOffset = TypeOffset;
					ChunkInfo.DataOffset = MetadataPos;
					ChunkInfo.SizeInBytes = ChunkSize;

					FLocalFileReplayDataInfo& DataInfo = TaskReplayInfo.DataChunks.AddDefaulted_GetRef();
					DataInfo.ChunkIndex = TaskReplayInfo.Chunks.Num() - 1;
					DataInfo.Time1 = Time1;
					DataInfo.Time2 = Time2;
					DataInfo.SizeInBytes = SizeOnDisk;
					DataInfo.MemorySizeInBytes = SizeInMemory;
					DataInfo.ReplayDataOffset = ReplayDataOffset;
					DataInfo.StreamOffset = TaskReplayInfo.TotalDataSizeInBytes;
					
					TaskReplayInfo.TotalDataSizeInBytes += DataInfo.MemorySizeInBytes;
				}

				TaskReplayInfo.LengthInMS = TotalLengthInMS;

				WriteReplayInfo(*LocalFileAr.Get(), TaskReplayInfo);
			}
			else
			{
				ReplayResult = ELocalFileReplayResult::FileWriter;
			}
		},
		[this](ELocalFileReplayResult& ReplayResult)
		{
			if (ReplayResult == ELocalFileReplayResult::Success)
			{
				UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::None);
			}
			else
			{
				UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::FlushStream failed.");
				SetLastError(ReplayResult);
			}
		});

	StreamAr.Reset();

	// Keep track of the time range we have in our buffer, so we can accurately upload that each time we submit a chunk
	StreamTimeRange.Min = StreamTimeRange.Max;

	StreamChunkIndex++;

	LastChunkTime = FPlatformTime::Seconds();
}

void FLocalFileNetworkReplayStreamer::FlushCheckpoint(const uint32 TimeInMS)
{
	if (CheckpointAr.TotalSize() == 0)
	{
		UE_LOGF( LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::FlushCheckpoint. Checkpoint is empty." );
		return;
	}

	// Flush any existing stream, we need checkpoints to line up with the next chunk
	FlushStream(TimeInMS);

	// Flush the checkpoint
	FlushCheckpointInternal(TimeInMS);
}

void FLocalFileNetworkReplayStreamer::FlushCheckpointInternal(const uint32 TimeInMS)
{
	if (StreamerState != EReplayStreamerState::Recording || CheckpointAr.TotalSize() == 0)
	{
		// If there is no active session, or we are not recording, we don't need to flush
		CheckpointAr.Buffer.Empty();
		CheckpointAr.Seek(0);
		return;
	}

	const int32 TotalLengthInMS = CurrentReplayInfo.LengthInMS;
	const uint32 CheckpointTimeInMS = StreamTimeRange.Max;

	AddGenericRequestToQueue<ELocalFileReplayResult>(EQueuedLocalFileRequestType::WritingCheckpoint, 
		[this, CheckpointTimeInMS, TotalLengthInMS, CheckpointData=MoveTemp(CheckpointAr.Buffer)](ELocalFileReplayResult& ReplayResult) mutable
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_FlushCheckpoint);

			const int32 DataChunkIndex = TaskReplayInfo.DataChunks.Num();
			const int32 CheckpointIndex = TaskReplayInfo.Checkpoints.Num();

			if (LocalFileAr.IsValid())
			{
				LocalFileAr->Seek(LocalFileAr->TotalSize());

				TArray<uint8> CompressedData;

				if (SupportsCompression())
				{
					SCOPE_CYCLE_COUNTER(STAT_LocalReplay_CompressTime);

					if (!CompressBuffer(CheckpointData, CompressedData))
					{
						UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::FlushStream - CompressBuffer failed");
						ReplayResult = ELocalFileReplayResult::CompressBuffer;
						return;
					}
				}
				else
				{
					CompressedData = MoveTemp(CheckpointData);
				}

				TArray<uint8> EncryptedData;

				if (AllowEncryptedWrite())
				{
					SCOPE_CYCLE_COUNTER(STAT_LocalReplay_EncryptTime);

					if (!EncryptBuffer(CompressedData, EncryptedData, TaskReplayInfo.EncryptionKey))
					{
						UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::FlushStream - EncryptBuffer failed");
						ReplayResult = ELocalFileReplayResult::EncryptBuffer;
						return;
					}
				}
				else
				{
					EncryptedData = MoveTemp(CompressedData);
				}

				// flush checkpoint
				if (EncryptedData.Num() > 0)
				{
					const int64 TypeOffset = LocalFileAr->Tell();

					ELocalFileChunkType ChunkType = ELocalFileChunkType::Checkpoint;
					*LocalFileAr << ChunkType;

					const int64 SavedPos = LocalFileAr->Tell();

					int32 PlaceholderSize = 0;
					*LocalFileAr << PlaceholderSize;

					const int64 MetadataPos = LocalFileAr->Tell();

					FString Id = FString::Printf(TEXT("checkpoint%ld"), CheckpointIndex);
					*LocalFileAr << Id;

					FString Group = "checkpoint";
					*LocalFileAr << Group;

					FString Metadata = FString::Printf(TEXT("%ld"), DataChunkIndex);
					*LocalFileAr << Metadata;

					uint32 Time1 = CheckpointTimeInMS;
					*LocalFileAr << Time1;

					uint32 Time2 = CheckpointTimeInMS;
					*LocalFileAr << Time2;

					int32 CheckpointSize = EncryptedData.Num();
					*LocalFileAr << CheckpointSize;

					const int64 EventDataOffset = LocalFileAr->Tell();

					LocalFileAr->Serialize((void*)EncryptedData.GetData(), EncryptedData.Num());

					int32 ChunkSize = IntCastChecked<int32>(LocalFileAr->Tell() - MetadataPos);

					LocalFileAr->Seek(SavedPos);
					*LocalFileAr << ChunkSize;

					FLocalFileChunkInfo& ChunkInfo = TaskReplayInfo.Chunks.AddDefaulted_GetRef();
					ChunkInfo.ChunkType = ChunkType;
					ChunkInfo.TypeOffset = TypeOffset;
					ChunkInfo.DataOffset = MetadataPos;
					ChunkInfo.SizeInBytes = ChunkSize;

					FLocalFileEventInfo& CheckpointInfo = TaskReplayInfo.Checkpoints.AddDefaulted_GetRef();
					CheckpointInfo.ChunkIndex = TaskReplayInfo.Chunks.Num() - 1;
					CheckpointInfo.Id = MoveTemp(Id);
					CheckpointInfo.Group = MoveTemp(Group);
					CheckpointInfo.Metadata = MoveTemp(Metadata);
					CheckpointInfo.Time1 = Time1;
					CheckpointInfo.Time2 = Time2;
					CheckpointInfo.SizeInBytes = CheckpointSize;
					CheckpointInfo.EventDataOffset = EventDataOffset;
				}

			}
			else
			{
				ReplayResult = ELocalFileReplayResult::FileWriter;
				return;
			}

			TaskReplayInfo.LengthInMS = TotalLengthInMS;

			WriteReplayInfo(*LocalFileAr.Get(), TaskReplayInfo);
		},
		[this](ELocalFileReplayResult& ReplayResult)
		{
			if (ReplayResult == ELocalFileReplayResult::Success)
			{
				UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::None);
			}
			else
			{
				UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::FlushCheckpointInternal failed.");
				SetLastError(ReplayResult);
			}
		});

	CheckpointAr.Buffer.Reset();
	CheckpointAr.Seek(0);
}

void FLocalFileNetworkReplayStreamer::GotoCheckpointIndex(const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType)
{
	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::ReadingCheckpoint))
	{
		// If we're currently going to a checkpoint now, ignore this request
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. Busy processing another checkpoint.");
		Delegate.ExecuteIfBound(FGotoResult());
		return;
	}

	if (CheckpointIndex == INDEX_NONE)
	{
		AddSimpleRequestToQueue(EQueuedLocalFileRequestType::ReadingCheckpoint, 
			[]()
			{
				UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex");
			},
			[this, Delegate]()
			{
				// Make sure to reset the checkpoint archive (this is how we signify that the engine should start from the beginning of the stream (we don't need a checkpoint for that))
				CheckpointAr.Reset();

				if (!IsDataAvailableForTimeRange(0, IntCastChecked<uint32>(LastGotoTimeInMS)))
				{
					// Completely reset our stream (we're going to start loading from the start of the checkpoint)
					StreamAr.Buffer.Reset();

					StreamDataOffset = 0;

					// Reset our stream range
					StreamTimeRange = TInterval<uint32>(0, 0);

					// Reset chunk index
					StreamChunkIndex = 0;

					LastChunkTime = 0;		// Force the next chunk to start loading immediately in case LastGotoTimeInMS is 0 (which would effectively disable high priority mode immediately)

					SetHighPriorityTimeRange(0, IntCastChecked<uint32>(LastGotoTimeInMS));
				}

				StreamAr.Seek(0);
				StreamAr.bAtEndOfReplay	= false;

				FGotoResult Result;
				Result.ExtraTimeMS = LastGotoTimeInMS;
				Result.Result = EStreamingOperationResult::Success;
				Result.CheckpointInfo.CheckpointIndex = FReplayCheckpointInfo::NO_CHECKPOINT;
				Result.CheckpointInfo.CheckpointStartTime = FReplayCheckpointInfo::NO_CHECKPOINT;

				Delegate.ExecuteIfBound( Result );

				LastGotoTimeInMS = -1;
			});
		
		return;
	}

	if (!CurrentReplayInfo.Checkpoints.IsValidIndex(CheckpointIndex))
	{
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. Invalid checkpoint index.");
		Delegate.ExecuteIfBound(FGotoResult());
		return;
	}

	if (CheckpointType == EReplayCheckpointType::Delta)
	{
		AddDelegateFileRequestToQueue<FGotoResult>(EQueuedLocalFileRequestType::ReadingCheckpoint,
			[this, CheckpointIndex](TLocalFileRequestCommonData<FGotoResult>& RequestData)
		{
			// If we get here after StopStreaming was called, then assume this operation should be cancelled
			// A more correct fix would be to actually cancel this in-flight request when StopStreaming is called
			// But for now, this is a safe change, and can co-exist with the more proper fix
			if (bStopStreamingCalled)
			{
				return;
			}

			RequestData.DataBuffer.Empty();

			const FString FullDemoFilename = GetDemoFullFilename(CurrentStreamName);

			TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullDemoFilename);
			if (LocalFileReader.IsValid())
			{
					TArray<uint8> CheckpointData;

					// read all the checkpoints
					for (int32 i = 0; i <= CheckpointIndex; ++i)
					{
						if (DeltaCheckpointCache.Contains(i))
						{
							FMemoryWriter Writer(RequestData.DataBuffer, true, true);
							uint32 CheckpointSize = DeltaCheckpointCache[i]->RequestData.Num();
							Writer << CheckpointSize;

							RequestData.DataBuffer.Append(DeltaCheckpointCache[i]->RequestData);
						}
						else
						{
							LocalFileReader->Seek(TaskReplayInfo.Checkpoints[i].EventDataOffset);

							CheckpointData.Reset();
							CheckpointData.AddUninitialized(TaskReplayInfo.Checkpoints[i].SizeInBytes);

							LocalFileReader->Serialize(CheckpointData.GetData(), CheckpointData.Num());

							TArray<uint8> PlaintextData;

							// Get the checkpoint data
							if (TaskReplayInfo.bEncrypted)
							{
								if (SupportsEncryption())
								{
									SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecryptTime);

									if (!DecryptBuffer(CheckpointData, PlaintextData, TaskReplayInfo.EncryptionKey))
									{
										UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. DecryptBuffer FAILED.");
										RequestData.DataBuffer.Empty();
										return;
									}
								}
								else
								{
									UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. Encrypted checkpoint but streamer does not support encryption.");
									RequestData.DataBuffer.Empty();
									return;
								}
							}
							else
							{
								PlaintextData = MoveTemp(CheckpointData);
							}

							TArray<uint8> UncompressedData;

							if (TaskReplayInfo.bCompressed)
							{
								if (SupportsCompression())
								{
									SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecompressTime);

									if (!DecompressBuffer(PlaintextData, UncompressedData))
									{
										UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. DecompressBuffer FAILED.");
										RequestData.DataBuffer.Empty();
										return;
									}
								}
								else
								{
									UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. Compressed checkpoint but streamer does not support compression.");
									RequestData.DataBuffer.Empty();
									return;
								}
							}
							else
							{
								UncompressedData = MoveTemp(PlaintextData);
							}

							FMemoryWriter Writer(RequestData.DataBuffer, true, true);
							uint32 CheckpointSize = UncompressedData.Num();
							Writer << CheckpointSize;

							RequestData.DataBuffer.Append(UncompressedData);

							DeltaCheckpointCache.Add(i, MakeShareable(new FCachedFileRequest(UncompressedData, 0)));
						}
					}

				LocalFileAr = nullptr;
			}
		},
			[this, CheckpointIndex, Delegate](TLocalFileRequestCommonData<FGotoResult>& RequestData)
		{
			if (bStopStreamingCalled)
			{
				Delegate.ExecuteIfBound(RequestData.DelegateResult);
				LastGotoTimeInMS = -1;
				return;
			}

			if (RequestData.DataBuffer.Num() == 0)
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. Checkpoint empty.");
				Delegate.ExecuteIfBound(RequestData.DelegateResult);
				LastGotoTimeInMS = -1;
				return;
			}

			CheckpointAr.Buffer = MoveTemp(RequestData.DataBuffer);
			CheckpointAr.Seek(0);

			const bool bIsDataAvailableForTimeRange = IsDataAvailableForTimeRange(CurrentReplayInfo.Checkpoints[CheckpointIndex].Time1, IntCastChecked<uint32>(LastGotoTimeInMS));

			const int32 DataChunkIndex = FCString::Atoi(*CurrentReplayInfo.Checkpoints[CheckpointIndex].Metadata);
			
			if (CurrentReplayInfo.DataChunks.IsValidIndex(DataChunkIndex))
			{
				if (!bIsDataAvailableForTimeRange)
				{
					// Completely reset our stream (we're going to start loading from the start of the checkpoint)
					StreamAr.Reset();

					// Reset any time we were waiting on in the past
					HighPriorityEndTime = 0;

					StreamDataOffset = CurrentReplayInfo.DataChunks[DataChunkIndex].StreamOffset;

					// Reset our stream range
					StreamTimeRange = TInterval<uint32>(0, 0);

					// Set the next chunk to be right after this checkpoint (which was stored in the metadata)
					StreamChunkIndex = DataChunkIndex;

					LastChunkTime = 0;		// Force the next chunk to start loading immediately in case LastGotoTimeInMS is 0 (which would effectively disable high priority mode immediately)
				}
				else
				{
					// set stream position back to the correct location
					StreamAr.Seek(CurrentReplayInfo.DataChunks[DataChunkIndex].StreamOffset - StreamDataOffset);
					check(StreamAr.Tell() >= 0 && StreamAr.Tell() <= StreamAr.TotalSize());
					StreamAr.bAtEndOfReplay = false;
				}
			}
			else
			{
				UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. Checkpoint data chunk index invalid: %d", DataChunkIndex);
				Delegate.ExecuteIfBound(RequestData.DelegateResult);
				LastGotoTimeInMS = -1;
				return;
			}

			// If we want to fast forward past the end of a stream (and we set a new chunk to stream), clamp to the checkpoint
			if (LastGotoTimeInMS >= 0 && StreamChunkIndex >= CurrentReplayInfo.DataChunks.Num() && !bIsDataAvailableForTimeRange)
			{
				UE_LOG(LogLocalFileReplay, Warning, TEXT("FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. Clamped to checkpoint: %" INT64_FMT), LastGotoTimeInMS);

				StreamTimeRange = TInterval<uint32>(CurrentReplayInfo.Checkpoints[CheckpointIndex].Time1, CurrentReplayInfo.Checkpoints[CheckpointIndex].Time1);
				LastGotoTimeInMS = -1;
			}

			if (LastGotoTimeInMS >= 0)
			{
				// If we are fine scrubbing, make sure to wait on the part of the stream that is needed to do this in one frame
				SetHighPriorityTimeRange(CurrentReplayInfo.Checkpoints[CheckpointIndex].Time1, IntCastChecked<uint32>(LastGotoTimeInMS));

				// Subtract off starting time so we pass in the leftover to the engine to fast forward through for the fine scrubbing part
				LastGotoTimeInMS -= CurrentReplayInfo.Checkpoints[CheckpointIndex].Time1;
			}

			// Notify game code of success
			RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
			RequestData.DelegateResult.ExtraTimeMS = LastGotoTimeInMS;

			Delegate.ExecuteIfBound(RequestData.DelegateResult);

			UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndexDelta. SUCCESS. StreamChunkIndex: %i", StreamChunkIndex);

			// Reset things
			LastGotoTimeInMS = -1;
		});
	}
	else
	{
		AddCachedFileRequestToQueue<FGotoResult>(EQueuedLocalFileRequestType::ReadingCheckpoint, CurrentReplayInfo.Checkpoints[CheckpointIndex].ChunkIndex,
			[this, CheckpointIndex](TLocalFileRequestCommonData<FGotoResult>& RequestData)
		{
			// If we get here after StopStreaming was called, then assume this operation should be cancelled
			// A more correct fix would be to actually cancel this in-flight request when StopStreaming is called
			// But for now, this is a safe change, and can co-exist with the more proper fix
			if (bStopStreamingCalled)
			{
				return;
			}

			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_ReadCheckpoint);

			RequestData.DataBuffer.Empty();

			const FString FullDemoFilename = GetDemoFullFilename(CurrentStreamName);

			TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullDemoFilename);
			if (LocalFileReader.IsValid())
			{
				LocalFileReader->Seek(TaskReplayInfo.Checkpoints[CheckpointIndex].EventDataOffset);

				RequestData.DataBuffer.AddUninitialized(TaskReplayInfo.Checkpoints[CheckpointIndex].SizeInBytes);

				LocalFileReader->Serialize(RequestData.DataBuffer.GetData(), RequestData.DataBuffer.Num());

					// Get the checkpoint data
					
				if (TaskReplayInfo.bEncrypted)
					{
						if (SupportsEncryption())
						{
							SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecryptTime);

							TArray<uint8> DecryptedData;

						if (!DecryptBuffer(RequestData.DataBuffer, DecryptedData, TaskReplayInfo.EncryptionKey))
							{
								UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. DecryptBuffer FAILED.");
								RequestData.DataBuffer.Empty();
								return;
							}

							RequestData.DataBuffer = MoveTemp(DecryptedData);
						}
						else
						{
							UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. Encrypted checkpoint but streamer does not support encryption.");
							RequestData.DataBuffer.Empty();
							return;
						}
					}

				if (TaskReplayInfo.bCompressed)
					{
						if (SupportsCompression())
						{
							SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecompressTime);

							TArray<uint8> UncompressedData;

							if (!DecompressBuffer(RequestData.DataBuffer, UncompressedData))
							{
								UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. DecompressBuffer FAILED.");
								RequestData.DataBuffer.Empty();
								return;
							}

							RequestData.DataBuffer = MoveTemp(UncompressedData);
						}
						else
						{
							UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. Compressed checkpoint but streamer does not support compression.");
							RequestData.DataBuffer.Empty();
							return;
						}
					}

				LocalFileReader = nullptr;
			}
		},
		[this, CheckpointIndex, Delegate](TLocalFileRequestCommonData<FGotoResult>& RequestData)
		{
			if (bStopStreamingCalled)
			{
				Delegate.ExecuteIfBound(RequestData.DelegateResult);
				LastGotoTimeInMS = -1;
				return;
			}

			if (RequestData.DataBuffer.Num() == 0)
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. Checkpoint empty.");
				Delegate.ExecuteIfBound(RequestData.DelegateResult);
				LastGotoTimeInMS = -1;
				return;
			}

			AddRequestToCache(CurrentReplayInfo.Checkpoints[CheckpointIndex].ChunkIndex, RequestData.DataBuffer);

			CheckpointAr.Buffer = MoveTemp(RequestData.DataBuffer);
			CheckpointAr.Seek(0);

			const FLocalFileEventInfo& Checkpoint = CurrentReplayInfo.Checkpoints[CheckpointIndex];
			const int32 DataChunkIndex = FCString::Atoi(*Checkpoint.Metadata);

			if (CurrentReplayInfo.DataChunks.IsValidIndex(DataChunkIndex))
			{
				bool bIsDataAvailableForTimeRange = IsDataAvailableForTimeRange(Checkpoint.Time1, IntCastChecked<uint32>(LastGotoTimeInMS));

				if (!bIsDataAvailableForTimeRange)
				{
					// Completely reset our stream (we're going to start loading from the start of the checkpoint)
					StreamAr.Reset();

					// Reset any time we were waiting on in the past
					HighPriorityEndTime = 0;

					StreamDataOffset = CurrentReplayInfo.DataChunks[DataChunkIndex].StreamOffset;

					// Reset our stream range
					StreamTimeRange = TInterval<uint32>(0, 0);

					// Set the next chunk to be right after this checkpoint (which was stored in the metadata)
					StreamChunkIndex = DataChunkIndex;

					LastChunkTime = 0;		// Force the next chunk to start loading immediately in case LastGotoTimeInMS is 0 (which would effectively disable high priority mode immediately)
				}
				else
				{
					// set stream position back to the correct location
					StreamAr.Seek(CurrentReplayInfo.DataChunks[DataChunkIndex].StreamOffset - StreamDataOffset);
					check(StreamAr.Tell() >= 0 && StreamAr.Tell() <= StreamAr.TotalSize());
					StreamAr.bAtEndOfReplay = false;
				}
			}
			else if (LastGotoTimeInMS >= 0)
			{
				UE_LOG(LogLocalFileReplay, Warning, TEXT("FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. Clamped to checkpoint: %" INT64_FMT), LastGotoTimeInMS);

				// If we want to fast forward past the end of a stream, clamp to the checkpoint
				StreamTimeRange = TInterval<uint32>(Checkpoint.Time1, Checkpoint.Time1);
				LastGotoTimeInMS = -1;
			}

			if (LastGotoTimeInMS >= 0)
			{
				// If we are fine scrubbing, make sure to wait on the part of the stream that is needed to do this in one frame
				SetHighPriorityTimeRange(Checkpoint.Time1, IntCastChecked<uint32>(LastGotoTimeInMS));

				// Subtract off starting time so we pass in the leftover to the engine to fast forward through for the fine scrubbing part
				LastGotoTimeInMS -= Checkpoint.Time1;
			}

			// Notify game code of success
			RequestData.DelegateResult.Result = EStreamingOperationResult::Success;
			RequestData.DelegateResult.ExtraTimeMS = LastGotoTimeInMS;
			RequestData.DelegateResult.CheckpointInfo.CheckpointIndex = CheckpointIndex;
			RequestData.DelegateResult.CheckpointInfo.CheckpointStartTime = Checkpoint.Time1;

			Delegate.ExecuteIfBound(RequestData.DelegateResult);

			UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::GotoCheckpointIndex. SUCCESS. StreamChunkIndex: %i", StreamChunkIndex);

			// Reset things
			LastGotoTimeInMS = -1;
		});
	}
}

void FLocalFileNetworkReplayStreamer::GotoTimeInMS(const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType)
{
	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::ReadingCheckpoint) || LastGotoTimeInMS != -1)
	{
		// If we're processing requests, be on the safe side and cancel the scrub
		// FIXME: We can cancel the in-flight requests as well
		UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::GotoTimeInMS. Busy processing pending requests.");
		Delegate.ExecuteIfBound( FGotoResult() );
		return;
	}

	UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::GotoTimeInMS. TimeInMS: %i", (int)TimeInMS );

	check(LastGotoTimeInMS == -1);

	int32 CheckpointIndex = -1;

	LastGotoTimeInMS = FMath::Min( TimeInMS, (uint32)CurrentReplayInfo.LengthInMS );

	if (CurrentReplayInfo.Checkpoints.Num() > 0 && TimeInMS >= CurrentReplayInfo.Checkpoints[CurrentReplayInfo.Checkpoints.Num() - 1].Time1)
	{
		// If we're after the very last checkpoint, that's the one we want
		CheckpointIndex = CurrentReplayInfo.Checkpoints.Num() - 1;
	}
	else
	{
		// Checkpoints should be sorted by time, return the checkpoint that exists right before the current time
		// For fine scrubbing, we'll fast forward the rest of the way
		// NOTE - If we're right before the very first checkpoint, we'll return -1, which is what we want when we want to start from the very beginning
		for (int32 i = 0; i < CurrentReplayInfo.Checkpoints.Num(); i++)
		{
			if (TimeInMS < CurrentReplayInfo.Checkpoints[i].Time1)
			{
				CheckpointIndex = i - 1;
				break;
			}
		}
	}

	GotoCheckpointIndex(CheckpointIndex, Delegate, CheckpointType);
}

bool FLocalFileNetworkReplayStreamer::HasPendingFileRequests() const
{
	// If there is currently one in progress, or we have more to process, return true
	return IsFileRequestInProgress() || QueuedRequests.Num() > 0;
}

bool FLocalFileNetworkReplayStreamer::IsFileRequestInProgress() const
{
	return ActiveRequest.IsValid();
}

bool FLocalFileNetworkReplayStreamer::IsFileRequestPendingOrInProgress(const EQueuedLocalFileRequestType::Type RequestType) const
{
	for (const TSharedPtr<FQueuedLocalFileRequest, ESPMode::ThreadSafe>& Request : QueuedRequests)
	{
		if (Request->GetRequestType() == RequestType)
		{
			return true;
		}
	}

	if (ActiveRequest.IsValid())
	{
		if (ActiveRequest->GetRequestType() == RequestType)
		{
			return true;
		}
	}

	return false;
}


bool FLocalFileNetworkReplayStreamer::ProcessNextFileRequest()
{
	LLM_SCOPE(ELLMTag::Replays);

	if (IsFileRequestInProgress())
	{
		return false;
	}

	if (QueuedRequests.Num() > 0)
	{
		TSharedPtr<FQueuedLocalFileRequest, ESPMode::ThreadSafe> QueuedRequest = QueuedRequests[0];
		QueuedRequests.RemoveAt( 0 );

		UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::ProcessNextFileRequest. Dequeue Type: %ls", EQueuedLocalFileRequestType::ToString(QueuedRequest->GetRequestType()));

		check( !ActiveRequest.IsValid() );

		ActiveRequest = QueuedRequest;

		if (ActiveRequest->GetCachedRequest())
		{
			ActiveRequest->FinishRequest();

			return ProcessNextFileRequest();
		}
		else
		{
			ActiveRequest->IssueRequest();
		}

		return true;
	}

	return false;
}

void FLocalFileNetworkReplayStreamer::Tick(float DeltaSeconds)
{
	// Attempt to process the next file request
	if (ProcessNextFileRequest())
	{
		check(IsFileRequestInProgress());
	}

	if (bStopStreamingCalled)
	{
		return;
	}

	if (StreamerState == EReplayStreamerState::Recording)
	{
		ConditionallyFlushStream();
	}
	else if (StreamerState == EReplayStreamerState::Playback)
	{
		if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::StartPlayback))
		{
			// If we're still waiting on finalizing the start request then return
			return;
		}

		// Check to see if we're done loading the high priority portion of the stream
		// If so, we can cancel the request
		if (HighPriorityEndTime > 0 && StreamTimeRange.Contains(HighPriorityEndTime))
		{
			HighPriorityEndTime = 0;
		}

		// Check to see if we're at the end of non live streams
		if (StreamChunkIndex >= CurrentReplayInfo.DataChunks.Num() && !CurrentReplayInfo.bIsLive)
		{
			// Make note of when we reach the end of non live stream
			StreamAr.bAtEndOfReplay = true;
		}

		ConditionallyLoadNextChunk();
		ConditionallyRefreshReplayInfo();
	}
}

const TArray<uint8>& FLocalFileNetworkReplayStreamer::GetCachedFileContents(const FString& Filename) const
{
	TArray<uint8>& Data = FileContentsCache.FindOrAdd(Filename);
	if (Data.Num() == 0)
	{
		// Read the whole file into memory
		FArchive* Ar = IFileManager::Get().CreateFileReader(*Filename, FILEREAD_AllowWrite);
		if (Ar)
		{
			Data.AddUninitialized(IntCastChecked<int32>(Ar->TotalSize()));
			Ar->Serialize(Data.GetData(), Data.Num());
			delete Ar;
		}
	}

	return Data;
}

TSharedPtr<FArchive> FLocalFileNetworkReplayStreamer::CreateLocalFileReader(const FString& InFilename) const
{
	LLM_SCOPE(ELLMTag::Replays);

	if (bCacheFileReadsInMemory)
	{
		const TArray<uint8>& Data = GetCachedFileContents(InFilename);
		return (Data.Num() > 0) ? MakeShareable(new FLargeMemoryReader((uint8*)Data.GetData(), Data.Num())) : nullptr;
	}
	else
	{
		return MakeShareable(IFileManager::Get().CreateFileReader(*InFilename, FILEREAD_AllowWrite));
	}
}

TSharedPtr<FArchive> FLocalFileNetworkReplayStreamer::CreateLocalFileWriter(const FString& InFilename) const
{
	return MakeShareable(IFileManager::Get().CreateFileWriter(*InFilename, FILEWRITE_Append | FILEWRITE_AllowRead));
}

TSharedPtr<FArchive> FLocalFileNetworkReplayStreamer::CreateLocalFileWriterForOverwrite(const FString& InFilename) const
{
	return MakeShareable(IFileManager::Get().CreateFileWriter(*InFilename, FILEWRITE_AllowRead));
}

FString FLocalFileNetworkReplayStreamer::GetDemoPath() const
{
	return DemoSavePath;
}

FString FLocalFileNetworkReplayStreamer::GetDemoFullFilename(const FString& StreamName) const
{
	// call the static version
	return GetDemoFullFilename(GetDemoPath(), StreamName);
}

/* static */FString FLocalFileNetworkReplayStreamer::GetDemoFullFilename(const FString& DemoPath, const FString& StreamName) 
{
	if (FPaths::IsRelative(StreamName))
	{
		// Treat relative paths as demo stream names.
		return FPaths::Combine(*DemoPath, *StreamName) + FNetworkReplayStreaming::GetReplayFileExtension();
	}
	else
	{
		// Return absolute paths without modification.
		return StreamName;
	}
}

bool FLocalFileNetworkReplayStreamer::GetDemoFreeStorageSpace(uint64& DiskFreeSpace, const FString& DemoPath)
{
#if PLATFORM_USE_PLATFORM_FILE_MANAGED_STORAGE_WRAPPER
	int64 AllocatedUsed = 0;
	int64 AllocatedFree = 0;
	int64 AllocatedTotal = 0;
	bool bManagedStorageQueryResult = FPersistentStorageManager::Get().GetPersistentStorageUsage(DemoPath, AllocatedUsed, AllocatedFree, AllocatedTotal);

	if (bManagedStorageQueryResult)
	{
		if (AllocatedTotal >= 0) // If total space is < 0, then the storage category is unlimited, so fall back to a physical free space check
		{
			DiskFreeSpace = 0;
			if (ensure(AllocatedFree >= 0))
			{
				DiskFreeSpace = (uint64)AllocatedFree;
			}
			return true;
		}
	}
	else
	{
		UE_LOGF(LogLocalFileReplay, Log, "Failed to get persistent storage useage for %ls from the FPeristentStorageManager, falling back to global total disk size", *DemoPath);
	}
#endif

	uint64 TotalDiskSpace = 0;
	uint64 TotalDiskFreeSpace = 0;
	bool bActualStorageQueryResult = FPlatformMisc::GetDiskTotalAndFreeSpace(DemoPath, TotalDiskSpace, TotalDiskFreeSpace);

	if (!bActualStorageQueryResult)
	{
		// This initial call to GetDiskTotalAndFreeSpace can fail if no replay has been recorded before and the demo folder doesn't exist
		// so try creating the folder.
		IFileManager& FileManager = IFileManager::Get();
		if (!FileManager.DirectoryExists(*DemoPath) && FileManager.MakeDirectory(*DemoPath, true))
		{
			TotalDiskSpace = 0;
			TotalDiskFreeSpace = 0;
			bActualStorageQueryResult = FPlatformMisc::GetDiskTotalAndFreeSpace(DemoPath, TotalDiskSpace, TotalDiskFreeSpace);
		}

		if (!bActualStorageQueryResult)
		{
			UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::GetDemoFreeStorageSpace. Unable to determine free space in %ls.", *DemoPath);
			return false;
		}
	}

	DiskFreeSpace = TotalDiskFreeSpace;
	return true;
}

bool FLocalFileNetworkReplayStreamer::CleanUpOldReplays(const FString& DemoPath, TArrayView<const FString> AdditionalRelativeDemoPaths)
{
	const int32 MaxDemos = FNetworkReplayStreaming::GetMaxNumberOfAutomaticReplays();
	const bool bUnlimitedDemos = (MaxDemos <= 0);
	const bool bUseDatePostfix = FNetworkReplayStreaming::UseDateTimeAsAutomaticReplayPostfix();
	const FString AutoPrefix = FNetworkReplayStreaming::GetAutomaticReplayPrefix();

	IFileManager& FileManager = IFileManager::Get();

	if (!bUnlimitedDemos)
	{
		uint64 TotalDiskFreeSpace = 0;

		if (!GetDemoFreeStorageSpace(TotalDiskFreeSpace, DemoPath))
		{
			// if we fail to get the storage space this is likely because the directory doesn't exist
			return true;
		}

		uint64 MinFreeSpace = UE::Net::LocalFileReplay::CVarReplayRecordingMinSpace.GetValueOnAnyThread();

		// build an array of replay info sorted by timestamps
		struct FAutoReplayInfo
		{
			FString		Path;
			FDateTime	TimeStamp;

			// sort by timestamp (reverse order to get newest->oldest)
			bool operator<(const FAutoReplayInfo& Other) const
			{
				return Other.TimeStamp < TimeStamp;
			}
		};

		// All the replays in the base DemoPath come first
		TArray<FAutoReplayInfo> SortedAutoReplays;
		{
			const FString WildCardPath = GetDemoFullFilename(DemoPath, AutoPrefix + FString("*"));

			TArray<FString> FoundAutoReplays;
			FileManager.FindFiles(FoundAutoReplays, *WildCardPath, /* bFiles= */ true, /* bDirectories= */ false);

			SortedAutoReplays.Reserve(SortedAutoReplays.Num() + FoundAutoReplays.Num());
			for (const FString& AutoReplay : FoundAutoReplays)
			{
				// Rebuild full path
				FString AutoReplayPath = FPaths::Combine(DemoPath, AutoReplay);
				FDateTime Timestamp = FileManager.GetTimeStamp(*AutoReplayPath);
				SortedAutoReplays.Add({ MoveTemp(AutoReplayPath), Timestamp });
			}
		}
		SortedAutoReplays.Sort();

		// remove oldest replays until we have enough space to record again and are below the MaxDemos threshold
		while (SortedAutoReplays.Num() &&
			((TotalDiskFreeSpace < MinFreeSpace) || (SortedAutoReplays.Num() >= MaxDemos)))
		{
			// find and delete the oldest replay
			const FAutoReplayInfo OldestReplay = SortedAutoReplays.Pop(EAllowShrinking::No);

			// Try deleting the replay
			if (!ensureMsgf(FileManager.Delete(*OldestReplay.Path, /*bRequireExists=*/ true, /*bEvenIfReadOnly=*/ true), TEXT("FLocalFileNetworkReplayStreamer::CleanUpOldReplays: Failed to delete old replay %s"), *OldestReplay.Path))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::CleanUpOldReplays. Unable to delete old replay %ls.", *OldestReplay.Path);
				return false;
			}

			if (!GetDemoFreeStorageSpace(TotalDiskFreeSpace, DemoPath))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::CleanUpOldReplays. Unable to refresh free space in %ls.", *DemoPath);
				return false;
			}
		}

		if (TotalDiskFreeSpace >= MinFreeSpace)
		{
			return true;
		}

		// Only delete additional replays if there is still not enough space
		TArray<FAutoReplayInfo> SortedAdditionalAutoReplays;
		for (const FString& RelPath : AdditionalRelativeDemoPaths)
		{
			const FString Path = FPaths::Combine(DemoPath, RelPath);
			const FString AdditionalWildCardPath = GetDemoFullFilename(Path, TEXT("*"));

			TArray<FString> FoundAdditionalReplays;
			FileManager.FindFiles(FoundAdditionalReplays, *AdditionalWildCardPath, /* bFiles= */ true, /* bDirectories= */ false);

			SortedAdditionalAutoReplays.Reserve(SortedAdditionalAutoReplays.Num() + FoundAdditionalReplays.Num());
			for (const FString& AutoReplay : FoundAdditionalReplays)
			{
				// Rebuild full path
				FString AutoReplayPath = FPaths::Combine(Path, AutoReplay);
				FDateTime Timestamp = FileManager.GetTimeStamp(*AutoReplayPath);
				SortedAdditionalAutoReplays.Add({ MoveTemp(AutoReplayPath), Timestamp });
			}
		}
		SortedAdditionalAutoReplays.Sort();

		// remove oldest replays until we have enough space to record again
		while (SortedAdditionalAutoReplays.Num() && (TotalDiskFreeSpace < MinFreeSpace))
		{
			// find and delete the oldest replay
			const FAutoReplayInfo OldestReplay = SortedAdditionalAutoReplays.Pop(EAllowShrinking::No);

			// Try deleting the replay
			if (!ensureMsgf(FileManager.Delete(*OldestReplay.Path, /*bRequireExists=*/ true, /*bEvenIfReadOnly=*/ true), TEXT("FLocalFileNetworkReplayStreamer::CleanUpOldReplays: Failed to delete old replay %s"), *OldestReplay.Path))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::CleanUpOldReplays. Unable to delete old replay %ls.", *OldestReplay.Path);
				return false;
			}

			if (!GetDemoFreeStorageSpace(TotalDiskFreeSpace, DemoPath))
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::CleanUpOldReplays. Unable to refresh free space in %ls.", *DemoPath);
				return false;
			}
		}

		return TotalDiskFreeSpace >= MinFreeSpace;
	}
	return true;
}

FString FLocalFileNetworkReplayStreamer::GetAutomaticDemoName() const
{
	SCOPE_CYCLE_COUNTER(STAT_LocalReplay_AutomaticName);

	const int32 MaxDemos = FNetworkReplayStreaming::GetMaxNumberOfAutomaticReplays();
	const bool bUnlimitedDemos = (MaxDemos <= 0);
	const bool bUseDatePostfix = FNetworkReplayStreaming::UseDateTimeAsAutomaticReplayPostfix();
	const FString AutoPrefix = FNetworkReplayStreaming::GetAutomaticReplayPrefix();

	IFileManager& FileManager = IFileManager::Get();

	if (bUseDatePostfix)
	{
		if (CleanUpOldReplays(GetDemoPath(), GetAdditionalRelativeDemoPaths()) == false)
		{
			return FString();
		}

		const FString FinalDemoName =
			UE::Net::LocalFileReplay::CVarbUsePlayerAccountIdPostfix->GetBool()
			? AutoPrefix + GetLocalPlayerUniqueNetId() + "-" + FDateTime::Now().ToString()
			: AutoPrefix + FDateTime::Now().ToString();

		UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::CleanUpOldReplays. FinalDemoName: %ls.", *FinalDemoName);

		return FinalDemoName;
	}
	else
	{
		FString FinalDemoName;
		FDateTime BestDateTime = FDateTime::MaxValue();

		int32 i = 1;
		while (bUnlimitedDemos || i <= MaxDemos)
		{
			const FString DemoName = FString::Printf(TEXT("%s%i"), *AutoPrefix, i);
			const FString FullDemoName = GetDemoFullFilename(DemoName);

			FDateTime DateTime = FileManager.GetTimeStamp(*FullDemoName);
			if (DateTime == FDateTime::MinValue())
			{
				// If we don't find this file, we can early out now
				FinalDemoName = DemoName;
				break;
			}
			else if (!bUnlimitedDemos && DateTime < BestDateTime)
			{
				// Use the oldest file
				FinalDemoName = DemoName;
				BestDateTime = DateTime;
			}

			++i;
		}

		UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::CleanUpOldReplays. FinalDemoName: %ls.", *FinalDemoName);

		return FinalDemoName;
	}
}

const FString& FLocalFileNetworkReplayStreamer::GetDefaultDemoSavePath()
{
	static const FString DefaultDemoSavePath = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Demos/"));
	return DefaultDemoSavePath;
}

FString FLocalFileNetworkReplayStreamer::GetLocalPlayerUniqueNetId() const
{
	if (GEngine)
	{
		if (UWorld* World = GWorld.GetReference())
		{
			if (const APlayerController* LocalPC = GEngine->GetFirstLocalPlayerController(World))
			{
				if (const ULocalPlayer* LocalPlayer = LocalPC->GetLocalPlayer())
				{
					const FUniqueNetIdRepl LocalPlayerNetId = LocalPlayer->GetPreferredUniqueNetId();
					if (LocalPlayerNetId.IsValid())
					{
						return LocalPlayerNetId->ToString();
					}
				}
			}
		}
	}

	return FString();
}

uint32 FLocalFileNetworkReplayStreamer::GetMaxFriendlyNameSize() const
{
	return FLocalFileNetworkReplayStreamer::MaxFriendlyNameLen;
}

void FLocalFileNetworkReplayStreamer::DownloadHeader(const FDownloadHeaderCallback& Delegate)
{
	if (CurrentReplayInfo.bIsValid && CurrentReplayInfo.Chunks.IsValidIndex(CurrentReplayInfo.HeaderChunkIndex))
	{
		const FLocalFileChunkInfo& ChunkInfo = CurrentReplayInfo.Chunks[CurrentReplayInfo.HeaderChunkIndex];

		int64 HeaderOffset = ChunkInfo.DataOffset;
		int32 HeaderSize = ChunkInfo.SizeInBytes;

		AddDelegateFileRequestToQueue<FDownloadHeaderResult>(EQueuedLocalFileRequestType::ReadingHeader,
			[this, HeaderSize, HeaderOffset](TLocalFileRequestCommonData<FDownloadHeaderResult>& RequestData)
			{
				SCOPE_CYCLE_COUNTER(STAT_LocalReplay_ReadHeader);

				TArray<uint8>& HeaderData = RequestData.DataBuffer;

				const FString FullDemoFilename = GetDemoFullFilename(CurrentStreamName);

				TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullDemoFilename);
				if (LocalFileReader.IsValid())
				{
					HeaderData.AddUninitialized(HeaderSize);

					LocalFileReader->Seek(HeaderOffset);
					LocalFileReader->Serialize((void*)HeaderData.GetData(), HeaderData.Num());

					RequestData.DelegateResult.Result = EStreamingOperationResult::Success;

					LocalFileReader = nullptr;
				}
			},
			[this, Delegate](TLocalFileRequestCommonData<FDownloadHeaderResult>& RequestData)
			{
				HeaderAr.Buffer = MoveTemp(RequestData.DataBuffer);
				HeaderAr.Seek(0);

				Delegate.ExecuteIfBound(RequestData.DelegateResult);
			});
	}
	else
	{
		Delegate.ExecuteIfBound(FDownloadHeaderResult());
	}
}

void FLocalFileNetworkReplayStreamer::WriteHeader()
{
	check(StreamAr.IsSaving());

	if (CurrentStreamName.IsEmpty())
	{
		// IF there is no active session, or we are not recording, we don't need to flush
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::WriteHeader. No session name!");
		return;
	}

	if (HeaderAr.TotalSize() == 0)
	{
		// Header wasn't serialized
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::WriteHeader. No header to upload");
		return;
	}

	if (!IsStreaming())
	{
		UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::WriteHeader. Not currently streaming");
		return;
	}

	AddGenericRequestToQueue<ELocalFileReplayResult>(EQueuedLocalFileRequestType::WritingHeader,
		[this, HeaderData=MoveTemp(HeaderAr.Buffer)](ELocalFileReplayResult& ReplayResult)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_FlushHeader);

			if (LocalFileAr.IsValid())
			{
				if (TaskReplayInfo.HeaderChunkIndex == INDEX_NONE)
				{
					// not expecting an existing header on disk, so check for it having been written by another process/client
					FLocalFileReplayInfo TestInfo;
					if (ReadReplayInfo(CurrentStreamName, TestInfo))
					{
						UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::WriteHeader - Current file already has unexpected header");
						ReplayResult = ELocalFileReplayResult::InvalidReplayInfo;
						return;
					}

					// append new chunk
					LocalFileAr->Seek(LocalFileAr->TotalSize());
				}
				else 
				{
					const int32 HeaderSize = TaskReplayInfo.Chunks[TaskReplayInfo.HeaderChunkIndex].SizeInBytes;

					LocalFileAr->Seek(TaskReplayInfo.Chunks[TaskReplayInfo.HeaderChunkIndex].TypeOffset);

					// reuse existing chunk if the size hasn't changed, otherwise, add a new one
					//@todo: add header data size, so we can differentiate and put a smaller header into the existing chunk space
					if (HeaderData.Num() != HeaderSize)
					{
						// clear chunk type so it will be skipped later
						ELocalFileChunkType ChunkType = ELocalFileChunkType::Unknown;
						*LocalFileAr << ChunkType;

						LocalFileAr->Seek(LocalFileAr->TotalSize());

						TaskReplayInfo.Chunks[TaskReplayInfo.HeaderChunkIndex].ChunkType = ChunkType;
						TaskReplayInfo.HeaderChunkIndex = INDEX_NONE;
					}
				}

				const int32 TypeOffset = IntCastChecked<int32>(LocalFileAr->Tell());

				ELocalFileChunkType ChunkType = ELocalFileChunkType::Header;
				*LocalFileAr << ChunkType;

				int32 ChunkSize = HeaderData.Num();
				*LocalFileAr << ChunkSize;

				const int32 DataOffset = IntCastChecked<int32>(LocalFileAr->Tell());

				LocalFileAr->Serialize((void*)HeaderData.GetData(), HeaderData.Num());

				if (TaskReplayInfo.HeaderChunkIndex == INDEX_NONE)
				{
					FLocalFileChunkInfo& ChunkInfo = TaskReplayInfo.Chunks.AddDefaulted_GetRef();
					ChunkInfo.ChunkType = ChunkType;
					ChunkInfo.TypeOffset = TypeOffset;
					ChunkInfo.DataOffset = DataOffset;
					ChunkInfo.SizeInBytes = ChunkSize;

					TaskReplayInfo.HeaderChunkIndex = TaskReplayInfo.Chunks.Num() - 1;
				}
			}
			else
			{
				ReplayResult = ELocalFileReplayResult::FileWriter;
			}
		},
		[this](ELocalFileReplayResult& ReplayResult)
		{
			if (ReplayResult == ELocalFileReplayResult::Success)
			{
				UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::None);
			}
			else
			{
				UE_LOGF(LogLocalFileReplay, Error, "FLocalFileNetworkReplayStreamer::WriteHeader failed.");
				SetLastError(ReplayResult);
			}
		});

	// We're done with the header archive
	HeaderAr.Reset();

	LastChunkTime = FPlatformTime::Seconds();
}

void FLocalFileNetworkReplayStreamer::RefreshHeader()
{
	AddSimpleRequestToQueue(EQueuedLocalFileRequestType::WriteHeader, 
		[]()
		{
			UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::RefreshHeader");
		},
		[this]()
		{
			WriteHeader();
		});
}

void FLocalFileNetworkReplayStreamer::SetHighPriorityTimeRange(const uint32 StartTimeInMS, const uint32 EndTimeInMS)
{
	HighPriorityEndTime = EndTimeInMS;
}

bool FLocalFileNetworkReplayStreamer::IsDataAvailableForTimeRange(const uint32 StartTimeInMS, const uint32 EndTimeInMS)
{
	if (HasError())
	{
		return false;
	}

	// If the time is within the stream range we have loaded, we will return true
	return (StreamTimeRange.Contains(StartTimeInMS) && StreamTimeRange.Contains(EndTimeInMS));
}

bool FLocalFileNetworkReplayStreamer::IsLoadingCheckpoint() const
{
	return IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::ReadingCheckpoint);
}

void FLocalFileNetworkReplayStreamer::OnFileRequestComplete(const TSharedPtr<FQueuedLocalFileRequest, ESPMode::ThreadSafe>& Request)
{
	if (Request.IsValid() && ActiveRequest.IsValid())
	{
		UE_LOGF(LogLocalFileReplay, Verbose, "FLocalFileNetworkReplayStreamer::OnFileRequestComplete. Type: %ls", EQueuedLocalFileRequestType::ToString(Request->GetRequestType()));

		ActiveRequest = nullptr;
	}
}

bool FLocalFileNetworkReplayStreamer::IsStreaming() const
{
	return (StreamerState != EReplayStreamerState::Idle);
}

void FLocalFileNetworkReplayStreamer::ConditionallyFlushStream()
{
	if ( IsFileRequestInProgress() || HasPendingFileRequests() )
	{
		return;
	}

	const float FLUSH_TIME_IN_SECONDS = UE::Net::LocalFileReplay::CVarChunkUploadDelayInSeconds.GetValueOnGameThread();

	if ( FPlatformTime::Seconds() - LastChunkTime > FLUSH_TIME_IN_SECONDS )
	{
		FlushStream(CurrentReplayInfo.LengthInMS);
	}
};

void FLocalFileNetworkReplayStreamer::ConditionallyLoadNextChunk()
{
	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::ReadingCheckpoint))
	{
		// Don't load a stream chunk while we're waiting for a checkpoint to load
		return;
	}

	if (IsFileRequestPendingOrInProgress(EQueuedLocalFileRequestType::ReadingStream))
	{
		// Only load one chunk at a time
		return;
	}

	const bool bMoreChunksDefinitelyAvailable = CurrentReplayInfo.DataChunks.IsValidIndex(StreamChunkIndex);	// We know for a fact there are more chunks available
	if (!bMoreChunksDefinitelyAvailable)
	{
		// don't read if no more chunks available, ConditionallyRefreshReplayInfo will refresh that data for us if bIsLive
		return;
	}

	// Determine if it's time to load the next chunk
	const bool bHighPriorityMode		= (HighPriorityEndTime > 0 && StreamTimeRange.Max < HighPriorityEndTime);			// We're within the high priority time range
	const bool bReallyNeedToLoadChunk	= bHighPriorityMode && bMoreChunksDefinitelyAvailable;

	// If it's not critical to load the next chunk (i.e. we're not scrubbing or at the end already), then check to see if we should grab the next chunk
	if (!bReallyNeedToLoadChunk)
	{
		const double LoadElapsedTime = FPlatformTime::Seconds() - LastChunkTime;

		// Unless it's critical (i.e. bReallyNeedToLoadChunk is true), never try faster than the min delay
		if (LoadElapsedTime < UE::Net::LocalFileReplay::CVarMinLoadNextChunkDelaySeconds.GetValueOnAnyThread())
		{
			return;		
		}

		if ((StreamTimeRange.Max > StreamTimeRange.Min) && (StreamAr.TotalSize() > 0))
		{
			// Make a guess on how far we're in
			const float PercentIn		= StreamAr.TotalSize() > 0 ? (float)StreamAr.Tell() / (float)StreamAr.TotalSize() : 0.0f;
			const float TotalStreamTimeSeconds = (float)(StreamTimeRange.Size()) / 1000.0f;
			const float CurrentTime		= TotalStreamTimeSeconds * PercentIn;
			const float TimeLeft		= TotalStreamTimeSeconds - CurrentTime;

			// Determine if we have enough buffer to stop streaming for now
			const float MaxBufferedTimeSeconds = UE::Net::LocalFileReplay::CVarChunkUploadDelayInSeconds.GetValueOnAnyThread() * 0.5f;

			if (TimeLeft > MaxBufferedTimeSeconds)
			{
				// Don't stream ahead by more than MaxBufferedTimeSeconds seconds
				UE_LOGF(LogLocalFileReplay, VeryVerbose, "ConditionallyLoadNextChunk. Cancelling due buffer being large enough. TotalStreamTime: %2.2f, PercentIn: %2.2f, TimeLeft: %2.2f", TotalStreamTimeSeconds, PercentIn, TimeLeft);
				return;
			}
		}
	}

	UE_LOGF(LogLocalFileReplay, Log, "FLocalFileNetworkReplayStreamer::ConditionallyLoadNextChunk. Index: %d", StreamChunkIndex);

	int32 RequestedStreamChunkIndex = StreamChunkIndex;

	AddCachedFileRequestToQueue<FStreamingResultBase>(EQueuedLocalFileRequestType::ReadingStream, CurrentReplayInfo.DataChunks[StreamChunkIndex].ChunkIndex,
		[this, RequestedStreamChunkIndex](TLocalFileRequestCommonData<FStreamingResultBase>& RequestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_LocalReplay_ReadStream);
			LLM_SCOPE(ELLMTag::Replays);

			check(TaskReplayInfo.DataChunks.IsValidIndex(RequestedStreamChunkIndex));

				RequestData.DataBuffer.Empty();
			
				const FString FullDemoFilename = GetDemoFullFilename(CurrentStreamName);
		
				TSharedPtr<FArchive> LocalFileReader = CreateLocalFileReader(FullDemoFilename);
				if (LocalFileReader.IsValid())
				{
					LocalFileReader->Seek(TaskReplayInfo.DataChunks[RequestedStreamChunkIndex].ReplayDataOffset);

				RequestData.DataBuffer.AddUninitialized(TaskReplayInfo.DataChunks[RequestedStreamChunkIndex].SizeInBytes);

				LocalFileReader->Serialize(RequestData.DataBuffer.GetData(), RequestData.DataBuffer.Num());

				if (TaskReplayInfo.bEncrypted)
					{
						if (SupportsEncryption())
						{
							SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecryptTime);

							TArray<uint8> DecryptedData;
						if (DecryptBuffer(RequestData.DataBuffer, DecryptedData, TaskReplayInfo.EncryptionKey))
							{
								RequestData.DataBuffer = MoveTemp(DecryptedData);
							}
							else
							{
								UE_LOGF(LogLocalFileReplay, Error, "ConditionallyLoadNextChunk failed to decrypt data.");
								RequestData.DataBuffer.Empty();
								RequestData.AsyncError = ELocalFileReplayResult::DecryptBuffer;
								return;
							}
						}
						else
						{
							UE_LOGF(LogLocalFileReplay, Error, "ConditionallyLoadNextChunk: Replay is marked encrypted but streamer does not support it.");
							RequestData.DataBuffer.Empty();
							RequestData.AsyncError = ELocalFileReplayResult::EncryptionNotSupported;
							return;
						}
					}

				if (TaskReplayInfo.bCompressed)
					{
						if (SupportsCompression())
						{
							SCOPE_CYCLE_COUNTER(STAT_LocalReplay_DecompressTime);

							TArray<uint8> UncompressedData;
							if (DecompressBuffer(RequestData.DataBuffer, UncompressedData))
							{
								RequestData.DataBuffer = MoveTemp(UncompressedData);
							}
							else
							{
								UE_LOGF(LogLocalFileReplay, Error, "ConditionallyLoadNextChunk failed to uncompresss data.");
								RequestData.DataBuffer.Empty();
								RequestData.AsyncError = ELocalFileReplayResult::DecompressBuffer;
								return;
							}
						}
						else
						{
							UE_LOGF(LogLocalFileReplay, Error, "ConditionallyLoadNextChunk: Replay is marked compressed but streamer does not support it.");
							RequestData.DataBuffer.Empty();
							RequestData.AsyncError = ELocalFileReplayResult::CompressionNotSupported;
							return;
						}
					}

					LocalFileReader = nullptr;
				}
		},
		[this, RequestedStreamChunkIndex](TLocalFileRequestCommonData<FStreamingResultBase>& RequestData)
		{
			LLM_SCOPE(ELLMTag::Replays);

			// Hijacking this error code to indicate a failure in encryption/compression
			if (RequestData.AsyncError != ELocalFileReplayResult::Success)
			{
				SetLastError(FLocalFileReplayResult(RequestData.AsyncError));
				return;
			}

			// Make sure our stream chunk index didn't change under our feet
			if (RequestedStreamChunkIndex != StreamChunkIndex)
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamer::ConditionallyLoadNextChunk failed with requested chunk index mismatch.");

				StreamAr.Reset();
				SetLastError(ELocalFileReplayResult::StreamChunkIndexMismatch);
				return;
			}

			if (RequestData.DataBuffer.Num() > 0)
			{
				if (StreamAr.TotalSize() == 0)
				{
					StreamTimeRange.Min = CurrentReplayInfo.DataChunks[RequestedStreamChunkIndex].Time1;
				}

				// This is the new end of the stream
				StreamTimeRange.Max = CurrentReplayInfo.DataChunks[RequestedStreamChunkIndex].Time2;

				check(StreamTimeRange.IsValid());

				// make space before appending
				const int32 MaxBufferedChunks = UE::Net::LocalFileReplay::CVarMaxBufferedStreamChunks.GetValueOnAnyThread();
				if (MaxBufferedChunks > 0)
				{
					const int32 MinChunkIndex = FMath::Max(0, (RequestedStreamChunkIndex + 1) - MaxBufferedChunks);
					if (MinChunkIndex > 0)
					{
						const int32 TrimBytes = IntCastChecked<int32>(CurrentReplayInfo.DataChunks[MinChunkIndex].StreamOffset - StreamDataOffset);
						if (TrimBytes > 0)
						{
							// can't remove chunks if we're actively seeking within that data
							if (StreamAr.Tell() >= TrimBytes)
							{
								// don't realloc, we're about to append anyway
								StreamAr.Buffer.RemoveAt(0, TrimBytes, EAllowShrinking::No);
								StreamAr.Seek(StreamAr.Tell() - TrimBytes);

								StreamTimeRange.Min = CurrentReplayInfo.DataChunks[MinChunkIndex].Time1;
								StreamDataOffset += TrimBytes;

								check(StreamTimeRange.IsValid());
							}
						}
					}
				}

				StreamAr.Buffer.Append(RequestData.DataBuffer);

				AddRequestToCache(CurrentReplayInfo.DataChunks[RequestedStreamChunkIndex].ChunkIndex, MoveTemp(RequestData.DataBuffer));

				StreamChunkIndex++;
			}
			else if (HighPriorityEndTime != 0)
			{
				// We failed to load live content during fast forward
				HighPriorityEndTime = 0;
			}
		});

	LastChunkTime = FPlatformTime::Seconds();
}

void FLocalFileNetworkReplayStreamer::ConditionallyRefreshReplayInfo()
{
	if (IsFileRequestInProgress() || HasPendingFileRequests())
	{
		return;
	}

	if (CurrentReplayInfo.bIsLive)
	{
		const double REFRESH_REPLAYINFO_IN_SECONDS = 10.0;

		if (FPlatformTime::Seconds() - LastRefreshTime > REFRESH_REPLAYINFO_IN_SECONDS)
		{
			const int64 LastDataSize = CurrentReplayInfo.TotalDataSizeInBytes;
			const FString FullDemoFilename = GetDemoFullFilename(CurrentStreamName);

			AddGenericRequestToQueue<ELocalFileReplayResult>(EQueuedLocalFileRequestType::RefreshingLiveStream, 
				[this](ELocalFileReplayResult& ReplayResult)
				{
					ReadReplayInfo(CurrentStreamName, TaskReplayInfo);
				},
				[this, LastDataSize](ELocalFileReplayResult& ReplayResult)
				{
					if (TaskReplayInfo.bIsValid && (TaskReplayInfo.TotalDataSizeInBytes != LastDataSize))
					{
						UpdateCurrentReplayInfo(TaskReplayInfo, EUpdateReplayInfoFlags::FullUpdate);
					}
				});

			LastRefreshTime = FPlatformTime::Seconds();
		}
	}
}

void FLocalFileNetworkReplayStreamer::AddRequestToCache(int32 ChunkIndex, const TArray<uint8>& RequestData)
{
	LLM_SCOPE(ELLMTag::Replays);
	TArray<uint8> DataCopy = RequestData;
	AddRequestToCache(ChunkIndex, MoveTemp(DataCopy));
}

void FLocalFileNetworkReplayStreamer::AddRequestToCache(int32 ChunkIndex, TArray<uint8>&& RequestData)
{
	if (!CurrentReplayInfo.bIsValid)
	{
		return;
	}

	if (!CurrentReplayInfo.Chunks.IsValidIndex(ChunkIndex))
	{
		return;
	}

	if (RequestData.Num() == 0)
	{
		return;
	}

	// Add to cache (or freshen existing entry)
	LLM_SCOPE(ELLMTag::Replays);
	RequestCache.Add(ChunkIndex, MakeShareable(new FCachedFileRequest(MoveTemp(RequestData), FPlatformTime::Seconds())));

	// Anytime we add something to cache, make sure it's within budget
	CleanupRequestCache();
}

void FLocalFileNetworkReplayStreamer::CleanupRequestCache()
{
	// Remove older entries until we're under the CVarMaxCacheSize threshold
	while (RequestCache.Num())
	{
		double OldestTime = 0.0;
		int32 OldestKey = INDEX_NONE;
		uint32 TotalSize = 0;

		for (const auto& RequestPair : RequestCache)
		{
			const TSharedPtr<FCachedFileRequest>& Request = RequestPair.Value;
			if (Request.IsValid())
			{
				if ((OldestKey == INDEX_NONE) || Request->LastAccessTime < OldestTime)
				{
					OldestTime = Request->LastAccessTime;
					OldestKey = RequestPair.Key;
				}

				// Accumulate total cache size
				TotalSize += Request->RequestData.Num();
			}
		}

		check(OldestKey != INDEX_NONE);

		const uint32 MaxCacheSize = UE::Net::LocalFileReplay::CVarMaxCacheSize.GetValueOnAnyThread();

		if (TotalSize <= MaxCacheSize)
		{
			break;	// We're good
		}

		RequestCache.Remove(OldestKey);
	}
}

void FQueuedLocalFileRequest::CancelRequest()
{
	bCancelled = true;
}

void FGenericQueuedLocalFileRequest::IssueRequest()
{
	auto SharedRef = AsShared();

	TGraphTask<TLocalFileAsyncGraphTask<void>>::CreateTask().ConstructAndDispatchWhenReady(
		[SharedRef]()
		{
			SharedRef->RequestFunction();
		},
		TPromise<void>([SharedRef]() 
		{
			if (!SharedRef->bCancelled)
			{
				AsyncTask(ENamedThreads::GameThread, [SharedRef]()
				{
					SharedRef->FinishRequest();
				});
			}
		})
	);
}

void FGenericQueuedLocalFileRequest::FinishRequest()
{
	if (!bCancelled && Streamer.IsValid())
	{
		if (CompletionCallback)
		{
			CompletionCallback();
		}

		Streamer->OnFileRequestComplete(AsShared());
	}
}

bool FLocalFileNetworkReplayStreamer::IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const
{
	bool bSupported = false;

	switch (CheckpointType)
	{
	case EReplayCheckpointType::Full:
	case EReplayCheckpointType::Delta:
		bSupported = true;
		break;
	}

	return bSupported;
}

void FLocalFileNetworkReplayStreamer::UpdateCurrentReplayInfo(FLocalFileReplayInfo& ReplayInfo, EUpdateReplayInfoFlags UpdateFlags)
{
	if (ensure(ReplayInfo.bIsValid))
	{
		// maintain the current values of the total length and encryption key
		TArray<uint8> CurrentKey = CurrentReplayInfo.EncryptionKey;
		const int32 TotalLengthInMS = CurrentReplayInfo.LengthInMS;

		CurrentReplayInfo = ReplayInfo;
		
		if (!EnumHasAnyFlags(UpdateFlags, EUpdateReplayInfoFlags::FullUpdate))
		{
			CurrentReplayInfo.LengthInMS = TotalLengthInMS;
			CurrentReplayInfo.EncryptionKey = MoveTemp(CurrentKey);
		}
	}
}

int32 FLocalFileNetworkReplayStreamer::GetDecompressedSizeBackCompat(FArchive& InCompressed) const
{
	return 0;
}

#define CASE_ELOCALFILEREPLAYRESULT_TO_TEXT_RET(txt) case txt: ReturnVal = TEXT(#txt); break;

const TCHAR* LexToString(ELocalFileReplayResult Enum)
{
	const TCHAR* ReturnVal = TEXT("::Invalid");

	switch (Enum)
	{
		FOREACH_ENUM_ELOCALFILEREPLAYRESULT(CASE_ELOCALFILEREPLAYRESULT_TO_TEXT_RET)
	}

	while (*ReturnVal != ':')
	{
		ReturnVal++;
	}

	ReturnVal += 2;

	return ReturnVal;
}

#undef CASE_ELOCALFILEREPLAYRESULT_TO_TEXT_RET

IMPLEMENT_MODULE(FLocalFileNetworkReplayStreamingFactory, LocalFileNetworkReplayStreaming)

TSharedPtr<INetworkReplayStreamer> FLocalFileNetworkReplayStreamingFactory::CreateReplayStreamer() 
{
	TSharedPtr<FLocalFileNetworkReplayStreamer> Streamer = MakeShared<FLocalFileNetworkReplayStreamer>();
	LocalFileStreamers.Add(Streamer);
	return Streamer;
}

void FLocalFileNetworkReplayStreamingFactory::Tick( float DeltaTime )
{
	for (int i = LocalFileStreamers.Num() - 1; i >= 0; i--)
	{
		check(LocalFileStreamers[i].IsValid());

		LocalFileStreamers[i]->Tick(DeltaTime);

		// We can release our hold when streaming is completely done
		if (LocalFileStreamers[i].IsUnique() && !LocalFileStreamers[i]->HasPendingFileRequests())
		{
			if (LocalFileStreamers[i]->IsStreaming())
			{
				UE_LOGF(LogLocalFileReplay, Warning, "FLocalFileNetworkReplayStreamingFactory::Tick. Stream was stopped early.");
			}

			LocalFileStreamers.RemoveAt(i);
		}
	}
}

bool FLocalFileNetworkReplayStreamingFactory::HasAnyPendingRequests() const
{
	bool bPendingRequests = false;

	for (const TSharedPtr<FLocalFileNetworkReplayStreamer>& Streamer : LocalFileStreamers)
	{
		if (Streamer.IsValid() && Streamer->HasPendingFileRequests())
		{
			bPendingRequests = true;
			break;
		}
	}

	return bPendingRequests;
}

void FLocalFileNetworkReplayStreamingFactory::Flush()
{
	bool bFlushStreamersOnShutdown = true;
	GConfig->GetBool(TEXT("LocalFileNetworkReplayStreamingFactory"), TEXT("bFlushStreamersOnShutdown"), bFlushStreamersOnShutdown, GEngineIni);

	if (bFlushStreamersOnShutdown)
	{
		double MaxFlushTimeSeconds = -1.0;
		GConfig->GetDouble(TEXT("LocalFileNetworkReplayStreamingFactory"), TEXT("MaxFlushTimeSeconds"), MaxFlushTimeSeconds, GEngineIni);

		double BeginWaitTime = FPlatformTime::Seconds();
		double LastTime = BeginWaitTime;
		while (HasAnyPendingRequests())
		{
			const double AppTime = FPlatformTime::Seconds();
			const double TotalWait = AppTime - BeginWaitTime;

			if ((MaxFlushTimeSeconds > 0) && (TotalWait > MaxFlushTimeSeconds))
			{
				UE_LOGF(LogLocalFileReplay, Display, "Abandoning streamer flush after waiting %0.2f seconds", TotalWait);
				break;
			}

			const float DeltaTime = FloatCastChecked<float>(AppTime - LastTime, UE::LWC::DefaultFloatPrecision);
			Tick(DeltaTime);

			LastTime = AppTime;

			if (HasAnyPendingRequests())
			{
				FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

				if (FPlatformProcess::SupportsMultithreading())
				{
					UE_LOGF(LogLocalFileReplay, Display, "Sleeping 0.1s to wait for outstanding requests.");
					FPlatformProcess::Sleep(0.1f);
				}
			}
		}
	}
}

void FLocalFileNetworkReplayStreamingFactory::StartupModule()
{
	FLocalFileNetworkReplayStreamer::CleanUpOldReplays(FLocalFileNetworkReplayStreamer::GetDefaultDemoSavePath());
}

void FLocalFileNetworkReplayStreamingFactory::ShutdownModule()
{
	Flush();
}

TStatId FLocalFileNetworkReplayStreamingFactory::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FLocalFileNetworkReplayStreamingFactory, STATGROUP_Tickables);
}
