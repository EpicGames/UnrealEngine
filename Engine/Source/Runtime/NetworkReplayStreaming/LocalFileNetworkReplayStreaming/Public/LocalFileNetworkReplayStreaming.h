// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NetworkReplayStreaming.h"
#include "Stats/Stats.h"
#include "Tickable.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MemoryReader.h"
#include "Async/Async.h"
#include "Templates/SharedPointer.h"
#include "HAL/ThreadSafeBool.h"
#include "LocalFileNetworkReplayStreaming.generated.h"

class FNetworkReplayVersion;
class FLocalFileNetworkReplayStreamer;

struct FLocalFileReplayCustomVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded = 0,

		FixedSizeFriendlyName = 1,
		CompressionSupport = 2,
		RecordingTimestamp = 3,
		StreamChunkTimes = 4,
		FriendlyNameCharEncoding = 5,
		EncryptionSupport = 6,
		CustomVersions = 7,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	LOCALFILENETWORKREPLAYSTREAMING_API const static FGuid Guid;

	FLocalFileReplayCustomVersion() = delete;
};

enum class ELocalFileChunkType : uint32
{
	Header,
	ReplayData,
	Checkpoint,
	Event,
	Unknown = 0xFFFFFFFF
};

enum class EReadReplayInfoFlags : uint32
{
	None = 0,
	SkipHeaderChunkTest = 1,
};

ENUM_CLASS_FLAGS(EReadReplayInfoFlags);

enum class EUpdateReplayInfoFlags : uint32
{
	None = 0,
	FullUpdate = 1,
};

ENUM_CLASS_FLAGS(EUpdateReplayInfoFlags);

/** Struct to hold chunk metadata */
struct FLocalFileChunkInfo
{
	FLocalFileChunkInfo() : 
		ChunkType(ELocalFileChunkType::Unknown),
		SizeInBytes(0),
		TypeOffset(0),
		DataOffset(0)
	{}

	ELocalFileChunkType ChunkType;
	int32 SizeInBytes;
	int64 TypeOffset;
	int64 DataOffset;
};

/** Struct to hold replay data chunk metadata */
struct FLocalFileReplayDataInfo
{
	FLocalFileReplayDataInfo() :
		ChunkIndex(INDEX_NONE),
		Time1(0),
		Time2(0),
		SizeInBytes(0),
		MemorySizeInBytes(0),
		ReplayDataOffset(0),
		StreamOffset(0)
	{}

	int32 ChunkIndex;
	uint32 Time1;
	uint32 Time2;
	int32 SizeInBytes;
	int32 MemorySizeInBytes;
	int64 ReplayDataOffset;
	int64 StreamOffset;
};

/** Struct to hold event metadata */
struct FLocalFileEventInfo
{
	FLocalFileEventInfo() : 
		ChunkIndex(INDEX_NONE),
		Time1(0),
		Time2(0),
		SizeInBytes(0),
		EventDataOffset(0)
	{}
	
	int32 ChunkIndex;

	FString Id;
	FString Group;
	FString Metadata;
	uint32 Time1;
	uint32 Time2;

	int32 SizeInBytes;
	int64 EventDataOffset;

	void CountBytes(FArchive& Ar) const;
};

/** Struct to hold metadata about an entire replay */
struct FLocalFileReplayInfo
{
	FLocalFileReplayInfo() :
		LengthInMS(0), 
		NetworkVersion(0),
		Changelist(0),
		Timestamp(0),
		TotalDataSizeInBytes(0),
		bIsLive(false),
		bIsValid(false),
		bCompressed(false),
		bEncrypted(false),
		HeaderChunkIndex(INDEX_NONE)
	{}

	int32 LengthInMS;
	uint32 NetworkVersion;
	uint32 Changelist;
	FString	FriendlyName;
	FDateTime Timestamp;
	int64 TotalDataSizeInBytes;
	bool bIsLive;
	bool bIsValid;
	bool bCompressed;

	bool bEncrypted;
	TArray<uint8> EncryptionKey;

	int32 HeaderChunkIndex;

	TArray<FLocalFileChunkInfo> Chunks;
	TArray<FLocalFileEventInfo> Checkpoints;
	TArray<FLocalFileEventInfo> Events;
	TArray<FLocalFileReplayDataInfo> DataChunks;

	void CountBytes(FArchive& Ar) const;
};

/** Archive to wrap the file reader and respect chunk boundaries */
class FLocalFileStreamFArchive : public FArchive
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FLocalFileStreamFArchive() : Pos(0), bAtEndOfReplay(false) {}
	FLocalFileStreamFArchive(const FLocalFileStreamFArchive&) = default;
	FLocalFileStreamFArchive& operator=(const FLocalFileStreamFArchive&) = default;
	LOCALFILENETWORKREPLAYSTREAMING_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void	Serialize(void* V, int64 Length) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual int64	Tell() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual int64	TotalSize() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void	Seek(int64 InPos) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual bool	AtEnd() override;

	LOCALFILENETWORKREPLAYSTREAMING_API int64 Tell() const;
	LOCALFILENETWORKREPLAYSTREAMING_API int64 TotalSize() const;

	void Reset()
	{
		Buffer.Reset();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Pos = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bAtEndOfReplay = false;
		ArchivePos = 0;
	}

	TArray<uint8>	Buffer;
	UE_DEPRECATED(5.3, "Please use Tell/Seek instead")
	int32			Pos;
	bool			bAtEndOfReplay;

private:
	int64 ArchivePos = 0;
};

namespace EQueuedLocalFileRequestType
{
	enum Type
	{
		StartRecording,
		WriteHeader,
		WritingHeader,
		WritingStream,
		StopRecording,
		StartPlayback,
		ReadingHeader,
		ReadingStream,
		EnumeratingStreams,
		WritingCheckpoint,
		ReadingCheckpoint,
		UpdatingEvent,
		EnumeratingEvents,
		RequestingEvent,
		StopStreaming,	
		DeletingFinishedStream,
		RefreshingLiveStream,
		KeepReplay,
		RenameReplay,
		RenameReplayFriendlyName,
	};

	inline const TCHAR* ToString( EQueuedLocalFileRequestType::Type Type )
	{
		switch ( Type )
		{
		case StartRecording:
			return TEXT( "StartRecording" );
		case WriteHeader:
			return TEXT( "WriteHeader" );
		case WritingHeader:
			return TEXT( "WritingHeader" );
		case WritingStream:
			return TEXT( "WritingStream" );
		case StopRecording:
			return TEXT( "StopRecording" );
		case StartPlayback:
			return TEXT( "StartPlayback" );
		case ReadingHeader:
			return TEXT( "ReadingHeader" );
		case ReadingStream:
			return TEXT( "ReadingStream" );
		case EnumeratingStreams:
			return TEXT( "EnumeratingStreams" );
		case WritingCheckpoint:
			return TEXT( "WritingCheckpoint" );
		case ReadingCheckpoint:
			return TEXT( "ReadingCheckpoint" );
		case UpdatingEvent:
			return TEXT( "UpdatingEvent" );
		case EnumeratingEvents:
			return TEXT( "EnumeratingEvents" );
		case RequestingEvent:
			return TEXT("RequestingEvent");
		case StopStreaming:
			return TEXT( "StopStreaming" );
		case DeletingFinishedStream:
			return TEXT( "DeletingFinishedStream" );
		case RefreshingLiveStream:
			return TEXT( "RefreshingLiveStream" );
		case KeepReplay:
			return TEXT( "KeepReplay" );
		case RenameReplay:
			return TEXT( "RenameReplay" );
		case RenameReplayFriendlyName:
			return TEXT( "RenameReplayFriendlyName" );
		}

		return TEXT( "Unknown EQueuedLocalFileRequestType type." );
	}
};

UENUM()
enum class ELocalFileReplayResult : uint32
{
	Success,
	InvalidReplayInfo,
	StreamChunkIndexMismatch,
	DecompressBuffer,
	CompressionNotSupported,
	DecryptBuffer,
	EncryptionNotSupported,
	EncryptBuffer,
	CompressBuffer,
	InvalidName,
	FileWriter,
	Unknown,
};

DECLARE_NETRESULT_ENUM(ELocalFileReplayResult);

LOCALFILENETWORKREPLAYSTREAMING_API const TCHAR* LexToString(ELocalFileReplayResult Enum);

class FCachedFileRequest
{
public:
	FCachedFileRequest(const TArray<uint8>& InRequestData, const double InLastAccessTime) 
		: RequestData(InRequestData)
		, LastAccessTime(InLastAccessTime)
	{
	}

	FCachedFileRequest(TArray<uint8>&& InRequestData, const double InLastAccessTime)
		: RequestData(MoveTemp(InRequestData))
		, LastAccessTime(InLastAccessTime)
	{
	}

	TArray<uint8> RequestData;
	double LastAccessTime;
};

class FQueuedLocalFileRequest
{
public:
	FQueuedLocalFileRequest(const TSharedPtr<FLocalFileNetworkReplayStreamer>& InStreamer, EQueuedLocalFileRequestType::Type InType)
		: Streamer(InStreamer)
		, RequestType(InType)
		, bCancelled(false)
	{
	}

	virtual ~FQueuedLocalFileRequest() = default;

	EQueuedLocalFileRequestType::Type GetRequestType() const { return RequestType; }

	virtual bool GetCachedRequest() { return false; }
	virtual void IssueRequest() = 0;
	virtual void FinishRequest() = 0;

	void CancelRequest();

protected:
	TSharedPtr<FLocalFileNetworkReplayStreamer> Streamer;
	EQueuedLocalFileRequestType::Type RequestType;
	FThreadSafeBool bCancelled;	
};

class FGenericQueuedLocalFileRequest : public FQueuedLocalFileRequest, public TSharedFromThis<FGenericQueuedLocalFileRequest, ESPMode::ThreadSafe>
{
public:
	FGenericQueuedLocalFileRequest(const TSharedPtr<FLocalFileNetworkReplayStreamer>& InStreamer, EQueuedLocalFileRequestType::Type InType, TFunction<void()>&& InFunction, TFunction<void()>&& InCompletionCallback)
		: FQueuedLocalFileRequest(InStreamer, InType)
		, RequestFunction(MoveTemp(InFunction))
		, CompletionCallback(MoveTemp(InCompletionCallback))
	{
	}

	virtual void IssueRequest() override;
	virtual void FinishRequest() override;

protected:
	TFunction<void()> RequestFunction;
	TFunction<void()> CompletionCallback;
};

template<typename ResultType>
class TLocalFileAsyncGraphTask : public FAsyncGraphTaskBase
{
public:
	TLocalFileAsyncGraphTask(TFunction<ResultType()>&& InFunction, TPromise<ResultType>&& InPromise)
		: Function(MoveTemp(InFunction))
		, Promise(MoveTemp(InPromise))
	{ }

public:

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		SetPromise(Promise, Function);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::AnyBackgroundThreadNormalTask;
	}

	TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(TLocalFileAsyncGraphTask, STATGROUP_TaskGraphTasks);
	}

	TFuture<ResultType> GetFuture()
	{
		return Promise.GetFuture();
	}

private:
	TFunction<ResultType()> Function;
	TPromise<ResultType> Promise;
};

template <typename StorageType>
class TGenericQueuedLocalFileRequest : public FQueuedLocalFileRequest, public TSharedFromThis<TGenericQueuedLocalFileRequest<StorageType>, ESPMode::ThreadSafe>
{
public:
	TGenericQueuedLocalFileRequest(const TSharedPtr<FLocalFileNetworkReplayStreamer>& InStreamer, EQueuedLocalFileRequestType::Type InType, TFunction<void(StorageType&)>&& InFunction, TFunction<void(StorageType&)>&& InCompletionCallback)
		: FQueuedLocalFileRequest(InStreamer, InType)
		, Storage()
		, RequestFunction(MoveTemp(InFunction))
		, CompletionCallback(MoveTemp(InCompletionCallback))
	{
	}

	virtual void IssueRequest() override
	{
		auto SharedRef = this->AsShared();

		TGraphTask<TLocalFileAsyncGraphTask<void>>::CreateTask().ConstructAndDispatchWhenReady(
			[SharedRef]()
			{
				SharedRef->RequestFunction(SharedRef->Storage);
			},
			TPromise<void>([SharedRef]() 
			{
				if (!SharedRef->bCancelled)
				{
					AsyncTask(ENamedThreads::GameThread, [SharedRef]()
					{
						SharedRef->FinishRequest();
					});
				}
			}));
	}

	virtual void FinishRequest() override
	{
		if (!bCancelled && this->Streamer.IsValid())
		{
			if (CompletionCallback)
			{
				CompletionCallback(Storage);
			}

			this->Streamer->OnFileRequestComplete(this->AsShared());
		}
	}

	StorageType Storage;

protected:
	TFunction<void(StorageType&)> RequestFunction;
	TFunction<void(StorageType&)> CompletionCallback;
};

template <typename DelegateResultType>
class TLocalFileRequestCommonData
{
public:
	DelegateResultType DelegateResult;
	
	UE_DEPRECATED(5.3, "No longer used")
	FLocalFileReplayInfo ReplayInfo;
	
	TArray<uint8> DataBuffer;

	UE_DEPRECATED(5.1, "No longer used")
	bool bAsyncError = false;

	ELocalFileReplayResult AsyncError = ELocalFileReplayResult::Success;
};	

template <typename DelegateResultType>
class TGenericCachedLocalFileRequest : public TGenericQueuedLocalFileRequest<TLocalFileRequestCommonData<DelegateResultType>>
{
public:
	TGenericCachedLocalFileRequest(int32 InCacheKey, const TSharedPtr<FLocalFileNetworkReplayStreamer>& InStreamer, EQueuedLocalFileRequestType::Type InType, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InFunction, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InCompletionCallback)
		: TGenericQueuedLocalFileRequest<TLocalFileRequestCommonData<DelegateResultType>>(InStreamer, InType, MoveTemp(InFunction), MoveTemp(InCompletionCallback))
		, CacheKey(InCacheKey)
	{}

	virtual bool GetCachedRequest() override
	{
		TSharedPtr<FCachedFileRequest> CachedRequest = this->Streamer->RequestCache.FindRef(CacheKey);
		if (CachedRequest.IsValid())
		{
			// If we have this response in the cache, process it now
			CachedRequest->LastAccessTime = FPlatformTime::Seconds();
			this->Storage.DataBuffer = CachedRequest->RequestData;
			return true;
		}

		return false; 
	}

protected:
	int32 CacheKey;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLocalFileReplayFinishedWriting, const FString& /*StreamName*/, const FString& /*FullReplayFile*/);

/** Local file streamer that supports playback/recording to a single file on disk */
class FLocalFileNetworkReplayStreamer : public INetworkReplayStreamer, public TSharedFromThis<FLocalFileNetworkReplayStreamer>
{
	using FLocalFileReplayResult = UE::Net::TNetResult<ELocalFileReplayResult>;

public:
	LOCALFILENETWORKREPLAYSTREAMING_API FLocalFileNetworkReplayStreamer();
	LOCALFILENETWORKREPLAYSTREAMING_API FLocalFileNetworkReplayStreamer(const FString& InDemoSavePath);
	LOCALFILENETWORKREPLAYSTREAMING_API virtual ~FLocalFileNetworkReplayStreamer();

	/** INetworkReplayStreamer implementation */
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void StopStreaming() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual FArchive* GetHeaderArchive() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual FArchive* GetStreamingArchive() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual FArchive* GetCheckpointArchive() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void FlushCheckpoint(const uint32 TimeInMS) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void GotoCheckpointIndex(const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void GotoTimeInMS(const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void UpdateTotalDemoTime(uint32 TimeInMS) override;
	virtual void UpdatePlaybackTime(uint32 TimeInMS) override {}
	virtual uint32 GetTotalDemoTime() const override { return CurrentReplayInfo.LengthInMS; }
	LOCALFILENETWORKREPLAYSTREAMING_API virtual bool IsDataAvailable() const override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void SetHighPriorityTimeRange(const uint32 StartTimeInMS, const uint32 EndTimeInMS) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual bool IsDataAvailableForTimeRange(const uint32 StartTimeInMS, const uint32 EndTimeInMS) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual bool IsLoadingCheckpoint() const override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual bool IsLive() const override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void DeleteFinishedStream(const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void DeleteFinishedStream( const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate ) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void EnumerateStreams( const FNetworkReplayVersion& InReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate ) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void EnumerateRecentStreams( const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate ) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void AddUserToReplay(const FString& UserString) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void AddEvent(const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void AddOrUpdateEvent(const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate ) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RequestEventData(const FString& ReplayName, const FString& EventId, const int32 UserIndex, const FRequestEventDataCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate) override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	virtual FString	GetReplayID() const override { return CurrentStreamName; }
	virtual EReplayStreamerState GetReplayStreamerState() const override { return StreamerState; }
	virtual void SetTimeBufferHintSeconds(const float InTimeBufferHintSeconds) override {}
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void RefreshHeader() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void DownloadHeader(const FDownloadHeaderCallback& Delegate) override;

	LOCALFILENETWORKREPLAYSTREAMING_API virtual bool IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const override;

	virtual bool SupportsCompression() const { return false; }

	virtual bool DecompressBuffer(const TArray<uint8>& InCompressed, TArray<uint8>& OutBuffer) const { return false; }
	virtual bool CompressBuffer(const TArray<uint8>& InBuffer, TArray<uint8>& OutCompressed) const { return false; }

	virtual bool SupportsEncryption() const { return false; }
	virtual void GenerateEncryptionKey(TArray<uint8>& EncryptionKey) {}
	virtual bool EncryptBuffer(TArrayView<const uint8> Plaintext, TArray<uint8>& Ciphertext, TArrayView<const uint8> EncryptionKey) const { return false; }
	virtual bool DecryptBuffer(TArrayView<const uint8> Ciphertext, TArray<uint8>& Plaintext, TArrayView<const uint8> EncryptionKey) const { return false; }

	LOCALFILENETWORKREPLAYSTREAMING_API bool AllowEncryptedWrite() const;

	LOCALFILENETWORKREPLAYSTREAMING_API void Tick(float DeltaSeconds);

	LOCALFILENETWORKREPLAYSTREAMING_API virtual uint32 GetMaxFriendlyNameSize() const override;

	virtual EStreamingOperationResult SetDemoPath(const FString& DemoPath) override
	{
		if (CurrentStreamName.IsEmpty())
		{
			DemoSavePath = DemoPath;
			return EStreamingOperationResult::Success;
		}
		else
		{
			return EStreamingOperationResult::Unspecified;
		}
	}

	virtual EStreamingOperationResult GetDemoPath(FString& DemoPath) const override
	{
		DemoPath = DemoSavePath;
		return EStreamingOperationResult::Success;
	}

	LOCALFILENETWORKREPLAYSTREAMING_API void OnFileRequestComplete(const TSharedPtr<FQueuedLocalFileRequest, ESPMode::ThreadSafe>& Request);

	LOCALFILENETWORKREPLAYSTREAMING_API bool IsStreaming() const;

	LOCALFILENETWORKREPLAYSTREAMING_API bool HasPendingFileRequests() const;

	void AddSimpleRequestToQueue(EQueuedLocalFileRequestType::Type RequestType, TFunction<void()>&& InFunction, TFunction<void()>&& InCompletionCallback)
	{
		QueuedRequests.Add(MakeShared<FGenericQueuedLocalFileRequest, ESPMode::ThreadSafe>(AsShared(), RequestType, MoveTemp(InFunction), MoveTemp(InCompletionCallback)));
	}

	template <typename StorageType>
	void AddGenericRequestToQueue(EQueuedLocalFileRequestType::Type RequestType, TFunction<void(StorageType&)>&& InFunction, TFunction<void(StorageType&)>&& InCompletionCallback)
	{
		QueuedRequests.Add(MakeShared<TGenericQueuedLocalFileRequest<StorageType>, ESPMode::ThreadSafe>(AsShared(), RequestType, MoveTemp(InFunction), MoveTemp(InCompletionCallback)));
	}

	template<typename DelegateResultType>
	void AddDelegateFileRequestToQueue(EQueuedLocalFileRequestType::Type RequestType, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InFunction, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InCompletionCallback)
	{
		AddGenericRequestToQueue<TLocalFileRequestCommonData<DelegateResultType>>(RequestType, MoveTemp(InFunction), MoveTemp(InCompletionCallback));
	}

	template<typename DelegateType, typename DelegateResultType>
	void AddDelegateFileRequestToQueue(EQueuedLocalFileRequestType::Type RequestType, const DelegateType& Delegate, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InFunction)
	{
		AddGenericRequestToQueue<TLocalFileRequestCommonData<DelegateResultType>>(RequestType, MoveTemp(InFunction), 
			[Delegate](TLocalFileRequestCommonData<DelegateResultType>& Storage) 
			{ 
				Delegate.ExecuteIfBound(Storage.DelegateResult); 
			});
	}

	template<typename DelegateResultType>
	void AddCachedFileRequestToQueue(EQueuedLocalFileRequestType::Type RequestType, int32 InCacheKey, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InFunction, TFunction<void(TLocalFileRequestCommonData<DelegateResultType>&)>&& InCompletionCallback)
	{
		QueuedRequests.Add(MakeShared<TGenericCachedLocalFileRequest<DelegateResultType>, ESPMode::ThreadSafe>(InCacheKey, AsShared(), RequestType, MoveTemp(InFunction), MoveTemp(InCompletionCallback)));
	}

	/** Current stream Archive */
	TSharedPtr<FArchive> LocalFileAr;

	/** Map of chunk index to cached value */
	TMap<int32, TSharedPtr<FCachedFileRequest>> RequestCache;

	/** Map of checkpoint index to cached value */
	TMap<int32, TSharedPtr<FCachedFileRequest>> DeltaCheckpointCache;

	static LOCALFILENETWORKREPLAYSTREAMING_API FOnLocalFileReplayFinishedWriting OnReplayFinishedWriting;

	TFunction<void(TFunction<void()> Complete)> PostHeaderDownloaded = [](TFunction<void()> Complete) { Complete(); };

protected:

	LOCALFILENETWORKREPLAYSTREAMING_API void DeleteFinishedStream_Internal(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate);
	LOCALFILENETWORKREPLAYSTREAMING_API void EnumerateEvents_Internal(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate);
	LOCALFILENETWORKREPLAYSTREAMING_API void RequestEventData_Internal(const FString& ReplayName, const FString& EventId, const int32 UserIndex, const FRequestEventDataCallback& Delegate);
	LOCALFILENETWORKREPLAYSTREAMING_API void KeepReplay_Internal(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate);
	LOCALFILENETWORKREPLAYSTREAMING_API void RenameReplayFriendlyName_Internal(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate);
	LOCALFILENETWORKREPLAYSTREAMING_API void RenameReplay_Internal(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate);

	/** 
	 * Currently playing or recording replay metadata
	 * 
	 * The values may not accurately reflect what is currently on disk during recording.  They will be updated 
	 * as each queued task completes on the game thread.
	 * 
	 **/
	FLocalFileReplayInfo CurrentReplayInfo;

	TInterval<uint32> StreamTimeRange;
	int64 StreamDataOffset;

	int32 StreamChunkIndex;
	double LastChunkTime;
	double LastRefreshTime;
	bool bStopStreamingCalled;
	uint32 HighPriorityEndTime;
	int64 LastGotoTimeInMS;

	TArray<TSharedPtr<FQueuedLocalFileRequest, ESPMode::ThreadSafe>> QueuedRequests;
	TSharedPtr<FQueuedLocalFileRequest, ESPMode::ThreadSafe> ActiveRequest;

	LOCALFILENETWORKREPLAYSTREAMING_API bool ProcessNextFileRequest();
	LOCALFILENETWORKREPLAYSTREAMING_API bool IsFileRequestInProgress() const;
	LOCALFILENETWORKREPLAYSTREAMING_API bool IsFileRequestPendingOrInProgress(const EQueuedLocalFileRequestType::Type RequestType) const;
	LOCALFILENETWORKREPLAYSTREAMING_API void CancelStreamingRequests();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "No longer used")
	void SetLastError(const ENetworkReplayError::Type InLastError) 
	{ 
		SetLastError(ELocalFileReplayResult::Unknown); 
	}
	LOCALFILENETWORKREPLAYSTREAMING_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void SetLastError(FLocalFileReplayResult&& Result);

	LOCALFILENETWORKREPLAYSTREAMING_API void ConditionallyFlushStream();
	LOCALFILENETWORKREPLAYSTREAMING_API void ConditionallyLoadNextChunk();
	LOCALFILENETWORKREPLAYSTREAMING_API void ConditionallyRefreshReplayInfo();

	LOCALFILENETWORKREPLAYSTREAMING_API void FlushCheckpointInternal(const uint32 TimeInMS);

	struct FLocalFileSerializationInfo
	{
		FLocalFileSerializationInfo();

		FLocalFileReplayCustomVersion::Type GetLocalFileReplayVersion() const;

		UE_DEPRECATED(5.2, "Replaced by FileCustomVersions.")
		uint32 FileVersion;

		FString FileFriendlyName;
		FCustomVersionContainer FileCustomVersions;
	};

	bool ReadReplayInfo(const FString& StreamName, FLocalFileReplayInfo& OutReplayInfo) const
	{
		return ReadReplayInfo(StreamName, OutReplayInfo, EReadReplayInfoFlags::None);
	}

	LOCALFILENETWORKREPLAYSTREAMING_API bool ReadReplayInfo(const FString& StreamName, FLocalFileReplayInfo& OutReplayInfo, EReadReplayInfoFlags Flags) const;
	LOCALFILENETWORKREPLAYSTREAMING_API bool ReadReplayInfo(FArchive& Archive, FLocalFileReplayInfo& OutReplayInfo, EReadReplayInfoFlags Flags) const;
	LOCALFILENETWORKREPLAYSTREAMING_API bool ReadReplayInfo(FArchive& Archive, FLocalFileReplayInfo& OutReplayInfo, struct FLocalFileSerializationInfo& SerializationInfo, EReadReplayInfoFlags Flags) const;

	LOCALFILENETWORKREPLAYSTREAMING_API bool WriteReplayInfo(FArchive& Archive, const FLocalFileReplayInfo& ReplayInfo);
	LOCALFILENETWORKREPLAYSTREAMING_API bool WriteReplayInfo(FArchive& Archive, const FLocalFileReplayInfo& InReplayInfo, struct FLocalFileSerializationInfo& SerializationInfo);

	LOCALFILENETWORKREPLAYSTREAMING_API void FixupFriendlyNameLength(const FString& UnfixedName, FString& FixedName) const;

	LOCALFILENETWORKREPLAYSTREAMING_API bool IsNamedStreamLive(const FString& StreamName) const;

	LOCALFILENETWORKREPLAYSTREAMING_API void FlushStream(const uint32 TimeInMS);

	LOCALFILENETWORKREPLAYSTREAMING_API void WriteHeader();

	LOCALFILENETWORKREPLAYSTREAMING_API virtual TSharedPtr<FArchive> CreateLocalFileReader(const FString& InFilename) const;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual TSharedPtr<FArchive> CreateLocalFileWriter(const FString& InFilename) const;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual TSharedPtr<FArchive> CreateLocalFileWriterForOverwrite(const FString& InFilename) const;

	LOCALFILENETWORKREPLAYSTREAMING_API FString GetDemoPath() const;
	// Must be relative to the base demo path
	virtual TArrayView<const FString> GetAdditionalRelativeDemoPaths() const { return {}; }
	LOCALFILENETWORKREPLAYSTREAMING_API FString GetDemoFullFilename(const FString& FileName) const;

	// Returns a name formatted as "demoX", where X is between 1 and MAX_DEMOS, inclusive.
	// Returns the first value that doesn't yet exist, or if they all exist, returns the oldest one
	// (it will be overwritten).
	LOCALFILENETWORKREPLAYSTREAMING_API FString GetAutomaticDemoName() const;

	FString GetLocalPlayerUniqueNetId() const;

	/** Handle to the archive that will read/write the demo header */
	FLocalFileStreamFArchive HeaderAr;

	/** Handle to the archive that will read/write network packets */
	FLocalFileStreamFArchive StreamAr;

	/* Handle to the archive that will read/write checkpoint files */
	FLocalFileStreamFArchive CheckpointAr;

	/** Overall state of the streamer */
	EReplayStreamerState StreamerState;

	/** Remember the name of the current stream, if any. */
	FString CurrentStreamName;

	FString DemoSavePath;

	LOCALFILENETWORKREPLAYSTREAMING_API void AddRequestToCache(int32 ChunkIndex, const TArray<uint8>& RequestData);
	LOCALFILENETWORKREPLAYSTREAMING_API void AddRequestToCache(int32 ChunkIndex, TArray<uint8>&& RequestData);
	LOCALFILENETWORKREPLAYSTREAMING_API void CleanupRequestCache();

	bool bCacheFileReadsInMemory;
	mutable TMap<FString, TArray<uint8>> FileContentsCache;
	LOCALFILENETWORKREPLAYSTREAMING_API const TArray<uint8>& GetCachedFileContents(const FString& Filename) const;

	LOCALFILENETWORKREPLAYSTREAMING_API void UpdateCurrentReplayInfo(FLocalFileReplayInfo& ReplayInfo, EUpdateReplayInfoFlags UpdateFlags = EUpdateReplayInfoFlags::None);

	LOCALFILENETWORKREPLAYSTREAMING_API virtual int32 GetDecompressedSizeBackCompat(FArchive& InCompressed) const;

public:
	static LOCALFILENETWORKREPLAYSTREAMING_API const FString& GetDefaultDemoSavePath();
	static LOCALFILENETWORKREPLAYSTREAMING_API FString GetDemoFullFilename(const FString& DemoPath, const FString& FileName);
	static LOCALFILENETWORKREPLAYSTREAMING_API bool CleanUpOldReplays(const FString& DemoPath = GetDefaultDemoSavePath(), TArrayView<const FString> AdditionalRelativeDemoPaths = {});
	static LOCALFILENETWORKREPLAYSTREAMING_API bool GetDemoFreeStorageSpace(uint64& DiskFreeSpace, const FString& DemoPath);

	static LOCALFILENETWORKREPLAYSTREAMING_API const uint32 FileMagic;
	static LOCALFILENETWORKREPLAYSTREAMING_API const uint32 MaxFriendlyNameLen;

	UE_DEPRECATED(5.2, "No longer used, replaced with custom version.")
	static LOCALFILENETWORKREPLAYSTREAMING_API const uint32 LatestVersion;

private:
	/** Manipulated by queued replay tasks, likely from another thread */
	FLocalFileReplayInfo TaskReplayInfo;
};

class FLocalFileNetworkReplayStreamingFactory : public INetworkReplayStreamingFactory, public FTickableGameObject
{
public:
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void StartupModule() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void ShutdownModule() override;

	LOCALFILENETWORKREPLAYSTREAMING_API virtual TSharedPtr<INetworkReplayStreamer> CreateReplayStreamer() override;
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void Flush() override;

	/** FTickableGameObject */
	LOCALFILENETWORKREPLAYSTREAMING_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	LOCALFILENETWORKREPLAYSTREAMING_API virtual TStatId GetStatId() const override;
	bool IsTickableWhenPaused() const override { return true; }

protected:
	LOCALFILENETWORKREPLAYSTREAMING_API bool HasAnyPendingRequests() const;

	TArray<TSharedPtr<FLocalFileNetworkReplayStreamer>> LocalFileStreamers;
};
