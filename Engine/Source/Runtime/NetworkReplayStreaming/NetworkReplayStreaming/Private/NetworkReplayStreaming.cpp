// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkReplayStreaming.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/ConsoleManager.h"

IMPLEMENT_MODULE( FNetworkReplayStreaming, NetworkReplayStreaming );

FOnReplayGetAnalyticsAttributes INetworkReplayStreamer::OnReplayGetAnalyticsAttributes;

INetworkReplayStreamingFactory& FNetworkReplayStreaming::GetFactory(const TCHAR* FactoryNameOverride)
{
	static const FString DefaultFactoryName = "LocalFileNetworkReplayStreaming";

	FString FactoryName = DefaultFactoryName;

	if (FactoryNameOverride == nullptr)
	{
		GConfig->GetString(TEXT("NetworkReplayStreaming"), TEXT("DefaultFactoryName"), FactoryName, GEngineIni);
	}
	else
	{
		FactoryName = FactoryNameOverride;
	}

	FString CmdlineFactoryName;
	if (FParse::Value(FCommandLine::Get(), TEXT("-REPLAYSTREAMER="), CmdlineFactoryName) || FParse::Value(FCommandLine::Get(), TEXT("-REPLAYSTREAMEROVERRIDE="), CmdlineFactoryName))
	{
		FactoryName = CmdlineFactoryName;
	}

	// See if we need to forcefully fallback to the null streamer
	if (!FModuleManager::Get().IsModuleLoaded(*FactoryName))
	{
		FModuleManager::Get().LoadModule(*FactoryName);
	
		if (!FModuleManager::Get().IsModuleLoaded(*FactoryName))
		{
			FactoryName = DefaultFactoryName;
		}
	}

	LoadedFactories.Add(*FactoryName);

	return FModuleManager::Get().LoadModuleChecked<INetworkReplayStreamingFactory>(*FactoryName);
}

int32 FNetworkReplayStreaming::GetMaxNumberOfAutomaticReplays()
{
	static const int32 DefaultMax = 10;

	int32 MaxAutomaticReplays = DefaultMax;
	GConfig->GetInt(TEXT("NetworkReplayStreaming"), TEXT("MaxNumberAutomaticReplays"), MaxAutomaticReplays, GEngineIni);

	if (!ensureMsgf(MaxAutomaticReplays >= 0, TEXT("INetworkReplayStreamer::GetMaxNumberOfAutomaticReplays: Invalid configured value, using default. %d"), MaxAutomaticReplays))
	{
		MaxAutomaticReplays = DefaultMax;
	}

	return MaxAutomaticReplays;
}

static TAutoConsoleVariable<FString> CVarReplayStreamerAutoDemoPrefix(
	TEXT("demo.ReplayStreamerAutoDemoPrefix"),
	FString("demo"),
	TEXT("Prefix to use when generating automatic demo names.")
);

static TAutoConsoleVariable<int32> CVarReplayStreamerAutoDemoUseDateTimePostfix(
	TEXT("demo.ReplayStreamerAutoDemoUseDateTimePostfix"),
	0,
	TEXT("When enabled, uses the current time as a postfix for automatic demo names instead of indices")
);

FString FNetworkReplayStreaming::GetAutomaticReplayPrefix()
{
	return CVarReplayStreamerAutoDemoPrefix.GetValueOnAnyThread();
}

FString FNetworkReplayStreaming::GetReplayFileExtension()
{
	return TEXT(".replay");
}

bool FNetworkReplayStreaming::UseDateTimeAsAutomaticReplayPostfix()
{
	return !!CVarReplayStreamerAutoDemoUseDateTimePostfix.GetValueOnAnyThread();
}

const FString FNetworkReplayStreaming::GetAutomaticReplayPrefixExtern() const
{
	return GetAutomaticReplayPrefix();
}

const int32 FNetworkReplayStreaming::GetMaxNumberOfAutomaticReplaysExtern() const
{
	return GetMaxNumberOfAutomaticReplays();
}

void FNetworkReplayStreaming::Flush()
{
	for (const FName& FactoryName : LoadedFactories)
	{
		if (FModuleManager::Get().IsModuleLoaded(FactoryName))
		{
			INetworkReplayStreamingFactory& ReplayFactory = FModuleManager::Get().LoadModuleChecked<INetworkReplayStreamingFactory>(FactoryName);
			ReplayFactory.Flush();
		}
	}
}

#if UE_ALLOW_EXEC_COMMANDS
bool FNetworkReplayStreaming::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// expected usage is "replaystreamer <streamer factory name> <streamer specific args>"
	if (FParse::Command(&Cmd, TEXT("REPLAYSTREAMER")))
	{
		FString FactoryName = FParse::Token(Cmd, false);
		if (!FactoryName.IsEmpty())
		{
			TSharedPtr<INetworkReplayStreamer> Streamer = GetFactory(*FactoryName).CreateReplayStreamer();
			if (Streamer.IsValid())
			{
				Streamer->Exec(Cmd, Ar);
			}
		}

		return true;
	}

	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

FString LexToString(const EReplayStreamerState State)
{
	FString Str;

	switch (State)
	{
	case EReplayStreamerState::Idle:
		Str = TEXT("Idle");
		break;
	case EReplayStreamerState::Recording:
		Str = TEXT("Recording");
		break;
	case EReplayStreamerState::Playback:
		Str = TEXT("Playback");
		break;
	default:
		Str = TEXT("Unknown");
		break;
	}

	return Str;
}

TArray<FAnalyticsEventAttribute> INetworkReplayStreamer::AppendCommonReplayAttributes(TArray<FAnalyticsEventAttribute>&& Attrs) const
{
	static const FString Attrib_StreamerState = "StreamerState";
	static const FString Attrib_ReplayName = "ReplayName";
	static const FString Attrib_LengthInMS = "LengthInMS";

	TArray<FAnalyticsEventAttribute> CommonAttributes = MoveTemp(Attrs);

	AppendAnalyticsEventAttributeArray(CommonAttributes,
		Attrib_StreamerState, LexToString(GetReplayStreamerState()),
		Attrib_ReplayName, GetReplayID(),
		Attrib_LengthInMS, GetTotalDemoTime()
	);

	INetworkReplayStreamer::OnReplayGetAnalyticsAttributes.Broadcast(this, CommonAttributes);

	return CommonAttributes;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ENetworkReplayError::Type INetworkReplayStreamer::GetLastError() const
{
	return HasError() ? ENetworkReplayError::ServiceUnavailable : ENetworkReplayError::None;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

INetworkReplayStreamer::INetworkReplayStreamer() = default;
INetworkReplayStreamer::~INetworkReplayStreamer() = default;

bool INetworkReplayStreamer::HasError() const
{
	return ExtendedError.IsValid();
}

void INetworkReplayStreamer::SetExtendedError(UE::Net::FNetResult&& Result)
{
	AddToChainResultPtr(ExtendedError, MoveTemp(Result));
}

UE::Net::EHandleNetResult INetworkReplayStreamer::HandleLastError(UE::Net::FNetResultManager& ResultManager)
{
	UE::Net::EHandleNetResult ReturnVal = UE::Net::EHandleNetResult::NotHandled;

	if (ExtendedError.IsValid())
	{
		ReturnVal = ResultManager.HandleNetResult(MoveTemp(*ExtendedError));
	}
	
	return ReturnVal;
}