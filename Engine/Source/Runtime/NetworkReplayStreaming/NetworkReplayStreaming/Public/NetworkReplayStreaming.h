// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Dependencies.

#include "AnalyticsEventAttribute.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/CoreMisc.h"
#include "Misc/DateTime.h"
#include "Misc/NetworkVersion.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/Core/Connection/NetResultManager.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FArchive;
class FOutputDevice;
class IAnalyticsProvider;
class UWorld;
struct FAnalyticsEventAttribute;

class FReplayEventListItem : public FJsonSerializable
{
public:
	FReplayEventListItem() : Time1(0), Time2(0) {}
	virtual ~FReplayEventListItem() = default;

	FString		ID;
	FString		Group;
	FString		Metadata;
	uint32		Time1;
	uint32		Time2;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("id", ID);
		JSON_SERIALIZE("group", Group);
		JSON_SERIALIZE("meta", Metadata);
		JSON_SERIALIZE("time1", Time1);
		JSON_SERIALIZE("time2", Time2);
	END_JSON_SERIALIZER
};

class FReplayEventList : public FJsonSerializable
{
public:
	FReplayEventList()
	{}
	virtual ~FReplayEventList() = default;

	TArray< FReplayEventListItem > ReplayEvents;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_ARRAY_SERIALIZABLE("events", ReplayEvents, FReplayEventListItem);
	END_JSON_SERIALIZER
};

/** Struct to store information about a stream, returned from search results. */
struct FNetworkReplayStreamInfo
{
	FNetworkReplayStreamInfo() : Timestamp( 0 ), SizeInBytes( 0 ), LengthInMS( 0 ), NumViewers( 0 ), Changelist( 0 ), bIsLive(false), bShouldKeep( false ) {}

	/** The name of the stream (generally this is auto generated, refer to friendly name for UI) */
	FString Name;

	/** The UI friendly name of the stream */
	FString FriendlyName;

	/** The date and time the stream was recorded */
	FDateTime Timestamp;

	/** The size of the stream */
	int64 SizeInBytes;
	
	/** The duration of the stream in MS */
	int32 LengthInMS;

	/** Number of viewers viewing this stream */
	int32 NumViewers;

	/** The changelist of the replay */
	int32 Changelist;

	/** True if the stream is live and the game hasn't completed yet */
	bool bIsLive;

	/** Debug feature that allows us to mark replays to not be deleted. True if this replay has been marked as such */
	bool bShouldKeep;
};

namespace ENetworkReplayError
{
	enum UE_DEPRECATED(5.1, "No longer used") Type
	{
		/** There are currently no issues */
		None,

		/** The backend service supplying the stream is unavailable, or connection interrupted */
		ServiceUnavailable,
	};

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline const TCHAR* ToString( const ENetworkReplayError::Type FailureType )
	{
		switch ( FailureType )
		{
			case None:
				return TEXT( "None" );
			case ServiceUnavailable:
				return TEXT( "ServiceUnavailable" );
		}
		return TEXT( "Unknown ENetworkReplayError error" );
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/**
 * Delegate called when StartStreaming() completes.
 *
 * @param bWasSuccessful Whether streaming was started.
 * @param bRecord Whether streaming is recording or not (vs playing)
 */
DECLARE_DELEGATE_TwoParams( FOnStreamReadyDelegate, const bool, const bool );

/**
 * Delegate called when GotoCheckpointIndex() completes.
 *
 * @param bWasSuccessful Whether streaming was started.
 */
DECLARE_DELEGATE_TwoParams( FOnCheckpointReadyDelegate, const bool, const int64 );

/**
 * Delegate called when DeleteFinishedStream() completes.
 *
 * @param bWasSuccessful Whether the stream was deleted.
 */
DECLARE_DELEGATE_OneParam( FOnDeleteFinishedStreamComplete, const bool );

/**
 * Delegate called when EnumerateStreams() completes.
 *
 * @param Streams An array containing information about the streams that were found.
 */
DECLARE_DELEGATE_OneParam( FOnEnumerateStreamsComplete, const TArray<FNetworkReplayStreamInfo>& );

/**
* Delegate called when EnumerateEvents() completes.
*
* @param ReplayEventList A list of events that were found
# @param bWasSuccessful Whether enumerating events was successful
*/
DECLARE_DELEGATE_TwoParams(FEnumerateEventsCompleteDelegate, const FReplayEventList&, bool);

/**
* Delegate called when RequestEventData() completes.
*
* @param ReplayEventListItem A replay event with its data parameter filled in
# @param bWasSuccessful Whether enumerating events was successful
*/
DECLARE_DELEGATE_TwoParams(FOnRequestEventDataComplete, const TArray<uint8>&, bool)

/**
* Delegate called when DownloadHeader() completes.
*
* @param bWasSuccessful Whether the header was successfully downloaded.
*/
DECLARE_DELEGATE_OneParam(FOnDownloadHeaderComplete, const bool);

/**
 * Below are all available new style delegate declarations for the INetworkReplayStreamer interface.
 *
 * All delegates should be named in the form F<MethodName>Callback.
 * All delegates should have an associated Result type.
 * All delegates should take one, and only one, argument which will be a const reference to the appropriate Result type.
 *
 * All result types should be named in the form F<MethodName>Result.
 * All result types should inherit from FStreamingResultBase.
 * All result types should be default constructible to a sensible error state, with OperationResult = Unspecified.
 *
 * FGotoCallback and FGotoResult are used for GotoCheckpointIndex and GotoTimeInMS, as they perform logically similar operations.
 * EnumerateRecentStreams uses the same delegate and result types as EnumerateStreams.
 */

/** Possible results for replay commands. */
enum class EStreamingOperationResult
{
	Success,		//! The operation succeeded.
	Unsupported,	//! The operation is not supported by the current streamer.
	ReplayNotFound,	//! The requested replay was not found.
	ReplayCorrupt,	//! The requested replay was found but was corrupt.
	NotEnoughSpace,	//! The operation failed due to insufficient storage space.
	NotEnoughSlots,	//! The operation failed due to reaching a predefined replay limit.
	Unspecified,	//! The operation failed for unspecified reasons.
	UnfinishedTask, //! The operation failed due to an outstanding task.
	EventNotFound,	//! The operation failed because the event could not be found.
	DecryptFailure,	//! The operation failed because decryption failed.
};

/**
 * Base type for all Streaming Operation results.
 * Should be used to store generic result information and convenience methods.
 */
struct FStreamingResultBase
{
	EStreamingOperationResult Result = EStreamingOperationResult::Unspecified;

	virtual ~FStreamingResultBase() = default;

	bool WasSuccessful() const
	{
		return EStreamingOperationResult::Success == Result;
	}
};

/** Start StartStreaming Types */
struct FStartStreamingResult : public FStreamingResultBase
{
	//! Whether or not Recording was requested (vs. Playback).
	bool bRecording = false;
};

DECLARE_DELEGATE_OneParam(FStartStreamingCallback, const FStartStreamingResult&);

static inline FStartStreamingCallback UpgradeStartStreamingDelegate(const FOnStreamReadyDelegate& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FStartStreamingCallback::CreateLambda([OldDelegate](const FStartStreamingResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.WasSuccessful(), Result.bRecording);
		});
	}
	else
	{
		return FStartStreamingCallback();
	}
}
/** End StartStreaming Types */

/** Start Goto Types */

//! Information about a checkpoint that was loaded as a result of a Goto Request.
struct FReplayCheckpointInfo
{
	//! Indicates an invalid checkpoint, or a bad request.
	static constexpr uint32 INVALID_CHECKPOINT = -2;

	//! Indicates that no checkpoint was loaded as a part of the request.
	//! This can happen if the requested time was before any available checkpoints.
	static constexpr uint32 NO_CHECKPOINT = -1;

	uint32 CheckpointStartTime = INVALID_CHECKPOINT;
	uint32 CheckpointIndex = INVALID_CHECKPOINT;
};

struct FGotoResult : public FStreamingResultBase
{
	//! Amount of extra time that the stream may need to be fast forwarded in order to reach
	//! the exact time specified (relative to the latest checkpoint before the specified time).
	int64 ExtraTimeMS = -1;

	//! Info about the checkpoint that was loaded as a result of the request.
	FReplayCheckpointInfo CheckpointInfo;
};

DECLARE_DELEGATE_OneParam(FGotoCallback, const FGotoResult&);

static inline FGotoCallback UpgradeGotoDelegate(const FOnCheckpointReadyDelegate& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FGotoCallback::CreateLambda([OldDelegate](const FGotoResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.WasSuccessful(), Result.ExtraTimeMS);
		});
	}
	else
	{
		return FGotoCallback();
	}
}
/** End Goto Types */

/** Start DeleteFinishedStream Types */
struct FDeleteFinishedStreamResult : public FStreamingResultBase
{
};

DECLARE_DELEGATE_OneParam(FDeleteFinishedStreamCallback, const FDeleteFinishedStreamResult&);

static inline FDeleteFinishedStreamCallback UpgradeDeleteFinishedStreamDelegate(const FOnDeleteFinishedStreamComplete& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FDeleteFinishedStreamCallback::CreateLambda([OldDelegate](const FDeleteFinishedStreamResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.WasSuccessful());
		});
	}
	else
	{
		return FDeleteFinishedStreamCallback();
	}
}
/** End DeleteFinishedStream Types */

/** Start EnumerateStreams Types */
struct FEnumerateStreamsResult : public FStreamingResultBase
{
	//! A list of streams that were found
	TArray<FNetworkReplayStreamInfo> FoundStreams;

	//! A list of streams (by name) that were found and were corrupted.
	TArray<FString> CorruptedStreams;
};

DECLARE_DELEGATE_OneParam(FEnumerateStreamsCallback, const FEnumerateStreamsResult&);

static inline FEnumerateStreamsCallback UpgradeEnumerateStreamsDelegate(const FOnEnumerateStreamsComplete& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FEnumerateStreamsCallback::CreateLambda([OldDelegate](const FEnumerateStreamsResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.FoundStreams);
		});
	}
	else
	{
		return FEnumerateStreamsCallback();
	}
}
/** End EnumerateStreams Types */

/** Start EnumerateEvents Types */
struct FEnumerateEventsResult : public FStreamingResultBase
{
	//! A list of events that were found
	FReplayEventList ReplayEventList;
};

DECLARE_DELEGATE_OneParam(FEnumerateEventsCallback, const FEnumerateEventsResult&);

static inline FEnumerateEventsCallback UpgradeEnumerateEventsDelegate(const FEnumerateEventsCompleteDelegate& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FEnumerateEventsCallback::CreateLambda([OldDelegate](const FEnumerateEventsResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.ReplayEventList, Result.WasSuccessful());
		});
	}
	else
	{
		return FEnumerateEventsCallback();
	}
}
/** End EnumerateEvents Types */

/** Start RequestEventData Types */
struct FRequestEventDataResult : public FStreamingResultBase
{
	//! A replay event with its data parameter filled in.
	TArray<uint8> ReplayEventListItem;
};

DECLARE_DELEGATE_OneParam(FRequestEventDataCallback, const FRequestEventDataResult&);

static inline FRequestEventDataCallback UpgradeRequestEventDelegate(const FOnRequestEventDataComplete& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FRequestEventDataCallback::CreateLambda([OldDelegate](const FRequestEventDataResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.ReplayEventListItem, Result.WasSuccessful());
		});
	}
	else
	{
		return FRequestEventDataCallback();
	}
}
/** End RequestEventData Types */

/** Start RequestEventGroupData Types */
struct FRequestEventGroupDataResult : public FStreamingResultBase
{
	TArray<FReplayEventListItem> ReplayEventListItems;
	TArray<FRequestEventDataResult> ReplayEventListResults;
};

DECLARE_DELEGATE_OneParam(FRequestEventGroupDataCallback, const FRequestEventGroupDataResult&);
/** End RequestEventGroupData Types */

/** Start DownloadHeader Types */
struct FDownloadHeaderResult : public FStreamingResultBase
{
};

DECLARE_DELEGATE_OneParam(FDownloadHeaderCallback, const FDownloadHeaderResult&);

static inline FDownloadHeaderCallback UpgradeDownloadHeaderDelegate(const FOnDownloadHeaderComplete& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FDownloadHeaderCallback::CreateLambda([OldDelegate](const FDownloadHeaderResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.WasSuccessful());
		});
	}
	else
	{
		return FDownloadHeaderCallback();
	}
}
/** End DownloadHeader Types */

/** Start SearchEvent Types */
struct FSearchEventsResult : public FStreamingResultBase
{
	// An array containing information about the streams that were found.
	TArray<FNetworkReplayStreamInfo> FoundStreams;

	//! A list of streams (by name) that were found and were corrupted.
	TArray<FString> CorruptedStreams;
};

DECLARE_DELEGATE_OneParam(FSearchEventsCallback, const FSearchEventsResult&);

static inline FSearchEventsCallback UpgradeSearchEventsDelegate(const FOnEnumerateStreamsComplete& OldDelegate)
{
	if (OldDelegate.IsBound())
	{
		return FSearchEventsCallback::CreateLambda([OldDelegate](const FSearchEventsResult& Result)
		{
			OldDelegate.ExecuteIfBound(Result.FoundStreams);
		});
	}
	else
	{
		return FSearchEventsCallback();
	}
}
/** End SearchEvent Types */

/** Start KeepReplay Types */
struct FKeepReplayResult : public FStreamingResultBase
{
	//! Saving the replay may cause the name to change.
	//! This points to the new name for the replay so it can be referenced in further operations.
	FString NewReplayName;

	//! Only valid if the error is NotEnoughSpace or NotEnoughSlots.
	//! For NotEnoughSpace, this will be the amount of storage space needed (in bytes) for the replay.
	//! For NotEnoughSlots, this will be the maximum number of slots that can be used to store replays.
	int64 RequiredSpace = 0;
};

DECLARE_DELEGATE_OneParam(FKeepReplayCallback, const FKeepReplayResult&);
/** End KeepReplay Types */

/** Start RenameReplay Types */
struct FRenameReplayResult : public FStreamingResultBase
{
};

DECLARE_DELEGATE_OneParam(FRenameReplayCallback, const FRenameReplayResult&);
/** End KeepReplay Types */

enum class EReplayCheckpointType : uint8
{
	Full,
	Delta
};

struct FStartStreamingParameters
{
	FString CustomName;
	FString FriendlyName;
	FString DemoURL;
	TArray<int32> UserIndices;
	FNetworkReplayVersion ReplayVersion;
	bool bRecord;
};

enum class EReplayStreamerState : uint8
{
	Idle,					// The streamer is idle. Either we haven't started activity yet, or we are done
	Recording,				// We are in the process of recording a replay
	Playback,				// We are in the process of playing a replay
};

FString NETWORKREPLAYSTREAMING_API LexToString(const EReplayStreamerState State);

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReplayGetAnalyticsAttributes, const class INetworkReplayStreamer* /*Streamer*/, TArray<FAnalyticsEventAttribute>& /*Attributes*/);

/**
 * Generic interface for network replay streaming
 *
 * When a delegate is provided as an argument, it is expected that the implementation calls
 * that delegate upon completion, and indicates success / failure through an appropriate
 * result type passed into the delegate.
 */
class INetworkReplayStreamer 
{
public:
	NETWORKREPLAYSTREAMING_API INetworkReplayStreamer();
	NETWORKREPLAYSTREAMING_API virtual ~INetworkReplayStreamer();

	virtual void StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate) = 0;

	virtual void StopStreaming() = 0;
	virtual FArchive* GetHeaderArchive() = 0;
	virtual FArchive* GetStreamingArchive() = 0;
	virtual FArchive* GetCheckpointArchive() = 0;
	virtual void FlushCheckpoint(const uint32 TimeInMS) = 0;

	virtual void GotoCheckpointIndex(const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType) = 0;

	virtual void GotoTimeInMS(const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType) = 0;

	virtual bool IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const = 0;

	virtual void UpdateTotalDemoTime(uint32 TimeInMS) = 0;
	virtual void UpdatePlaybackTime(uint32 TimeInMS) = 0;

	/** Time in milliseconds */
	virtual uint32 GetTotalDemoTime() const = 0;

	virtual bool IsDataAvailable() const = 0;
	virtual void SetHighPriorityTimeRange(const uint32 StartTimeInMS, const uint32 EndTimeInMS) = 0;
	virtual bool IsDataAvailableForTimeRange(const uint32 StartTimeInMS, const uint32 EndTimeInMS) = 0;
	virtual bool IsLoadingCheckpoint() const = 0;
	virtual void AddEvent(const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data) = 0;
	virtual void AddOrUpdateEvent(const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data) = 0;

	virtual void EnumerateEvents(const FString& Group, const FEnumerateEventsCallback& Delegate) = 0;

	virtual void EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate) = 0;
	virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate ) = 0;

	virtual void RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate) = 0;
	virtual void RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate) = 0;
	virtual void RequestEventData(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate) = 0;

	virtual void RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate) = 0;
	virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate) = 0;
	virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate) = 0;

	virtual void SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate) = 0;
	virtual void RefreshHeader() = 0;

	virtual void DownloadHeader(const FDownloadHeaderCallback& Delegate) = 0;

	/**
	 * Used to commit a replay to permanent storage.
	 *
	 * @param ReplayName Name of the replay to keep.
	 * @param bKeep Whether or not we actually want to keep this replay.
	 */
	virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate) = 0;
	virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate) = 0;

	/**
	 * Used to change the friendly name of a replay.
	 * Note, changing the friendly name **does not** change the name used to refer to replay.
	 *
	 * @param ReplayName Name of the replay to rename.
	 * @param NewFriendlyName The new friendly name for the replay.
	 */
	virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate) = 0;
	virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) = 0;

	/**
	 * Used to change the name of a replay.
	 * Note, this **will** change the name used to refer to the replay (if successful).
	 *
	 * @param ReplayName Name of the replay to rename.
	 * @param NewName The new name for the replay.
	 */
	virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate) = 0;
	virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate) = 0;

	/** Returns true if the playing stream is currently in progress */
	virtual bool IsLive() const = 0;

	/** Returns the active replay name */
	virtual FString	GetReplayID() const = 0;

	/** Return current recording/playback state */
	virtual EReplayStreamerState GetReplayStreamerState() const { return EReplayStreamerState::Idle; };

	/**
	 * Attempts to delete the stream with the specified name. May execute asynchronously.
	 *
	 * @param StreamName The name of the stream to delete
	 * @param Delegate A delegate that will be executed if bound when the delete operation completes
	 */
	virtual void DeleteFinishedStream(const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate) = 0;
	virtual void DeleteFinishedStream(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate) = 0;

	/**
	* Retrieves the streams that are available for viewing. May execute asynchronously.
	* Allows the caller to pass in a custom list of query parameters
	*/
	virtual void EnumerateStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate) = 0;

	/**
	 * Retrieves the streams that have been recently viewed. May execute asynchronously.
	 *
	 * @param Delegate A delegate that will be executed if bound when the list of streams is available
	 */
	virtual void EnumerateRecentStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate) = 0;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Returns the last error that occurred while streaming replays */
	UE_DEPRECATED(5.1, "Deprecated in favor of HasError and HandleLastError")
	NETWORKREPLAYSTREAMING_API virtual ENetworkReplayError::Type GetLastError() const;
	NETWORKREPLAYSTREAMING_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual bool HasError() const;

	NETWORKREPLAYSTREAMING_API virtual UE::Net::EHandleNetResult HandleLastError(UE::Net::FNetResultManager& ResultManager);

	NETWORKREPLAYSTREAMING_API virtual void SetExtendedError(UE::Net::FNetResult&& Result);

	/**
	 * Adds a join-in-progress user to the set of users associated with the currently recording replay (if any)
	 *
	 * @param UserString a string that uniquely identifies the user, usually their FUniqueNetId
	 */
	virtual void AddUserToReplay( const FString& UserString ) = 0;

	/**
	 * Sets a hint for how much data needs to be kept in memory. If set to a value greater than zero,
	 * a streamer implementation may free any in-memory data that would be required to go to a time
	 * before the beginning of the buffer.
	 */
	virtual void SetTimeBufferHintSeconds(const float InTimeBufferHintSeconds) = 0;

	/** Returns the maximum size of the friendly name text for this streamer, or 0 for unlimited. */
	virtual uint32 GetMaxFriendlyNameSize() const = 0;

	/**
	 * Changes the base directory where Demos are stored.
	 * Note, this will always fail for streamers that don't support replays stored on disk.
	 * This method should not be called after StartStreaming, or while async operations are pending on a streamer.
	 *
	 */
	virtual EStreamingOperationResult SetDemoPath(const FString& DemoPath) = 0;

	/**
	 * Gets the current base directory where Demos are stored.
	 * Note, this will always fail for streamers that don't support replays stored on disk.
	 */
	virtual EStreamingOperationResult GetDemoPath(FString& DemoPath) const = 0;

	virtual void SetAnalyticsProvider(TSharedPtr<IAnalyticsProvider>& InProvider) {}

	NETWORKREPLAYSTREAMING_API virtual TArray<FAnalyticsEventAttribute> AppendCommonReplayAttributes(TArray<FAnalyticsEventAttribute>&& Attrs) const;

	virtual void Exec(const TCHAR* Cmd, FOutputDevice& Ar) {}

	/** Called from base streamer interface AppendCommonReplayAttributes to set common attributes */
	static NETWORKREPLAYSTREAMING_API FOnReplayGetAnalyticsAttributes OnReplayGetAnalyticsAttributes;

private:
	TPimplPtr<UE::Net::FNetResult, EPimplPtrMode::DeepCopy> ExtendedError;
};

/** Replay streamer factory */
class INetworkReplayStreamingFactory : public IModuleInterface
{
public:
	virtual TSharedPtr<INetworkReplayStreamer> CreateReplayStreamer() = 0;
	virtual void Flush() {}
};

/** Replay streaming factory manager */
class FNetworkReplayStreaming : public IModuleInterface, public FSelfRegisteringExec
{
public:
	FNetworkReplayStreaming() {}
	virtual ~FNetworkReplayStreaming() = default;

	static inline FNetworkReplayStreaming& Get()
	{
		return FModuleManager::LoadModuleChecked<FNetworkReplayStreaming>("NetworkReplayStreaming");
	}

	NETWORKREPLAYSTREAMING_API void Flush();

	virtual INetworkReplayStreamingFactory& GetFactory(const TCHAR* FactoryNameOverride = nullptr);

	virtual const FString GetAutomaticReplayPrefixExtern() const;
	virtual const int32 GetMaxNumberOfAutomaticReplaysExtern() const;

	// Gets the configured max value for the number of automatic replays to support.
	// 0 indicates no limit.
	static NETWORKREPLAYSTREAMING_API int32 GetMaxNumberOfAutomaticReplays();

	// Gets the configured automatic replay name prefix.
	// The prefix should always be a non-empty string.
	// If using streamers that store replays on disk, this must also be comprised of only valid file name characters.
	static NETWORKREPLAYSTREAMING_API FString GetAutomaticReplayPrefix();

	// Gets the configured value for whether or not we should use FDateTime::Now as the automatic replay postfix.
	// If false, it's up to the streamer to determine a proper postfix.
	static NETWORKREPLAYSTREAMING_API bool UseDateTimeAsAutomaticReplayPostfix();

	// Gets the file extension for replay streamers to use when recording replay files to the local file system.
	// Includes the "." prefix
	static NETWORKREPLAYSTREAMING_API FString GetReplayFileExtension();

	// FSelfRegisteringExec interface
#if UE_ALLOW_EXEC_COMMANDS
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
#endif

private:
	TSet<FName> LoadedFactories;
};
