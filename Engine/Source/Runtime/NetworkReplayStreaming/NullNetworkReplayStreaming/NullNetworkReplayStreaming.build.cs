// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NullNetworkReplayStreaming : ModuleRules
	{
		public NullNetworkReplayStreaming( ReadOnlyTargetRules Target ) : base(Target)
		{
            ShortName = "NullReplayStreaming";

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Engine",
					"NetworkReplayStreaming",
                    "Json",
				}
			);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
