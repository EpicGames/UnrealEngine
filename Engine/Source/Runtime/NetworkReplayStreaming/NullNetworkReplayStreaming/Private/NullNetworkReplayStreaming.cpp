// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullNetworkReplayStreaming.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/EngineVersion.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/LocalPlayer.h"

DEFINE_LOG_CATEGORY_STATIC( LogNullReplay, Log, All );

/* Class to hold stream event information */
class FNullCheckpointListItem : public FJsonSerializable
{
public:
	FNullCheckpointListItem() : Time1(0), Time2(0) {}

	FString		Group;
	FString		Metadata;
	uint32		Time1;
	uint32		Time2;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE( "group",			Group );
		JSON_SERIALIZE( "meta",			Metadata );
		JSON_SERIALIZE( "time1",			Time1 );
		JSON_SERIALIZE( "time2",			Time2 );
	END_JSON_SERIALIZER
};

/**
 * Very basic implementation of network replay streaming using the file system
 * As of now, there is just simple opening and closing of the stream, and handing off the stream for direct use
 * Eventually, we'll want to expand this interface to allow enumerating demos, support for live spectating on local machine
 * (which will require support for writing/reading at the same time)
 */

static FString GetStreamBaseFilename(const FString& StreamName)
{
	int32 Year, Month, DayOfWeek, Day, Hour, Min, Sec, MSec;
	FPlatformTime::SystemTime( Year, Month, DayOfWeek, Day, Hour, Min, Sec, MSec );

	FString DemoName = StreamName;

	DemoName.ReplaceInline( TEXT( "%td" ), *FDateTime::Now().ToString() );
	DemoName.ReplaceInline( TEXT( "%d" ), *FString::Printf( TEXT( "%i-%i-%i" ), Month, Day, Year ) );
	DemoName.ReplaceInline( TEXT( "%t" ), *FString::Printf( TEXT( "%i" ), ( ( Hour * 3600 ) + ( Min * 60 ) + Sec ) * 1000 + MSec ) );
	DemoName.ReplaceInline( TEXT( "%v" ), *FString::Printf( TEXT( "%i" ), FEngineVersion::Current().GetChangelist() ) );

	// replace bad characters with underscores
	DemoName.ReplaceInline( TEXT( "\\" ),	TEXT( "_" ) );
	DemoName.ReplaceInline( TEXT( "/" ),	TEXT( "_" ) );
	DemoName.ReplaceInline( TEXT( "." ),	TEXT( "_" ) );
	DemoName.ReplaceInline( TEXT( " " ),	TEXT( "_" ) );
	DemoName.ReplaceInline( TEXT( "%" ),	TEXT( "_" ) );

	return DemoName;
}

static FString GetDemoPath()
{
	return FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT( "Demos/" ));
}

static FString GetStreamDirectory(const FString& StreamName)
{
	const FString DemoName = GetStreamBaseFilename(StreamName);

	// Directory for this demo
	const FString DemoDir  = FPaths::Combine(*GetDemoPath(), *DemoName);
	
	return DemoDir;
}

static FString GetStreamFullBaseFilename(const FString& StreamName)
{
	return FPaths::Combine(*GetStreamDirectory(StreamName), *GetStreamBaseFilename(StreamName));
}

static FString GetHeaderFilename(const FString& StreamName)
{
	return GetStreamFullBaseFilename(StreamName) + TEXT(".header");
}

static FString GetDemoFilename(const FString& StreamName)
{
	return GetStreamFullBaseFilename(StreamName) + TEXT(".demo");
}

static FString GetFinalFilename(const FString& StreamName)
{
	return GetStreamFullBaseFilename(StreamName) + TEXT(".final");
}

static FString GetCheckpointFilename( const FString& StreamName, int32 Index )
{
	return FPaths::Combine(*GetStreamDirectory(StreamName), TEXT("checkpoints"), *FString::Printf( TEXT("checkpoint%d"), Index ) );
}

static FString GetEventFilename( const FString& StreamName, int32 Index )
{
	return FPaths::Combine(*GetStreamDirectory(StreamName), TEXT("events"), *FString::Printf( TEXT("event%d"), Index ) );
}

static FString GetInfoFilename( const FString& StreamName )
{
	return GetStreamFullBaseFilename(StreamName) + TEXT(".replayinfo");
}

static FNullReplayInfo ReadReplayInfo( const FString& StreamName )
{
	FNullReplayInfo Info;

	const FString InfoFilename = GetInfoFilename(StreamName);
	TUniquePtr<FArchive> InfoFileArchive( IFileManager::Get().CreateFileReader( *InfoFilename ) );

	if ( InfoFileArchive.IsValid() && InfoFileArchive->TotalSize() != 0)
	{
		FString JsonString;
		*InfoFileArchive << JsonString;

		Info.FromJson(JsonString);
		Info.bIsValid = true;
	}

	return Info;
}

static void WriteReplayInfo( const FString& StreamName, const FNullReplayInfo& ReplayInfo )
{
	// Update metadata file with latest info
	TUniquePtr<FArchive> ReplayInfoFileAr(IFileManager::Get().CreateFileWriter(*GetInfoFilename(StreamName)));

	if (ReplayInfoFileAr.IsValid())
	{
		FString JsonString = ReplayInfo.ToJson();
		*ReplayInfoFileAr << JsonString; 
	}
}

// Returns a name formatted as "demoX", where X is 0-9.
// Returns the first value that doesn't yet exist, or if they all exist, returns the oldest one
// (it will be overwritten).
FString GetAutomaticDemoName()
{
	const int MaxDemos = FNetworkReplayStreaming::GetMaxNumberOfAutomaticReplays();
	const bool bUnlimitedDemos = MaxDemos == 0;

	FString FinalDemoName;
	FDateTime BestDateTime = FDateTime::MaxValue();

	int32 i = 1;
	while(bUnlimitedDemos || i <= MaxDemos)
	{
		const FString DemoName = FString::Printf(TEXT("demo%i"), i);
		const FString FullDemoName = GetDemoFilename(DemoName);
		
		FDateTime DateTime = IFileManager::Get().GetTimeStamp(*FullDemoName);

		if (DateTime == FDateTime::MinValue())
		{
			// If we don't find this file, we can early out now
			FinalDemoName = DemoName;
			break;
		}
		else if (!bUnlimitedDemos && DateTime < BestDateTime)
		{
			// Use the oldest file
			FinalDemoName = DemoName;
			BestDateTime = DateTime;
		}

		++i;
	}

	return FinalDemoName;
}

void FNullNetworkReplayStreamer::StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate)
{
	FString FinalDemoName = Params.CustomName;

	FStartStreamingResult Result;
	Result.bRecording = Params.bRecord;

	if ( Params.CustomName.IsEmpty() )
	{
		if ( Params.bRecord )
		{
			// If we're recording and the caller didn't provide a name, generate one automatically
			FinalDemoName = GetAutomaticDemoName();
		}
		else
		{
			// Can't play a replay if the user didn't provide a name!
			Result.Result = EStreamingOperationResult::ReplayNotFound;
			Delegate.ExecuteIfBound( Result );
			return;
		}
	}
	
	const FString DemoDir = GetStreamDirectory(FinalDemoName);

	const FString FullHeaderFilename = GetHeaderFilename(FinalDemoName);
	const FString FullDemoFilename = GetDemoFilename(FinalDemoName);
	
	CurrentStreamName = FinalDemoName;

	if ( !Params.bRecord )
	{
		// Load metadata if it exists
		ReplayInfo = ReadReplayInfo( CurrentStreamName );

		// Open file for reading
		ReopenStreamFileForReading();
		HeaderAr.Reset( IFileManager::Get().CreateFileReader( *FullHeaderFilename, FILEREAD_AllowWrite ) );
		StreamerState = EReplayStreamerState::Playback;
	}
	else
	{
		// Delete any existing demo with this name
		IFileManager::Get().DeleteDirectory( *DemoDir, false, true );
		
		// Create a directory for this demo
		IFileManager::Get().MakeDirectory( *DemoDir, true );

		// Open file for writing
		FileAr.Reset( IFileManager::Get().CreateFileWriter( *FullDemoFilename, FILEWRITE_AllowRead ) );
		HeaderAr.Reset( IFileManager::Get().CreateFileWriter( *FullHeaderFilename, FILEWRITE_AllowRead ) );
		StreamerState = EReplayStreamerState::Recording;

		CurrentCheckpointIndex = 0;

		// Set up replay info
		ReplayInfo.NetworkVersion = Params.ReplayVersion.NetworkVersion;
		ReplayInfo.Changelist = Params.ReplayVersion.Changelist;
		ReplayInfo.FriendlyName = Params.FriendlyName;

		WriteReplayInfo(CurrentStreamName, ReplayInfo);
	}

	// Notify immediately
	if (FileAr.Get() != nullptr && HeaderAr.Get() != nullptr)
	{
		Result.Result = EStreamingOperationResult::Success;
	}
	Delegate.ExecuteIfBound(Result);
}

void FNullNetworkReplayStreamer::StopStreaming()
{
	if (StreamerState == EReplayStreamerState::Recording)
	{
		WriteReplayInfo(CurrentStreamName, ReplayInfo);
	}

	TUniquePtr<FArchive> FinalFile( IFileManager::Get().CreateFileWriter( *GetFinalFilename( CurrentStreamName ) ) );

	HeaderAr.Reset();
	FileAr.Reset();

	CurrentStreamName.Empty();
	StreamerState = EReplayStreamerState::Idle;
}

FArchive* FNullNetworkReplayStreamer::GetHeaderArchive()
{
	return HeaderAr.Get();
}

FArchive* FNullNetworkReplayStreamer::GetStreamingArchive()
{
	return FileAr.Get();
}

void FNullNetworkReplayStreamer::UpdateTotalDemoTime(uint32 TimeInMS)
{
	check(StreamerState == EReplayStreamerState::Recording);

	ReplayInfo.LengthInMS = TimeInMS;
}

bool FNullNetworkReplayStreamer::IsDataAvailable() const
{
	check(StreamerState == EReplayStreamerState::Playback);

	return FileAr.IsValid() && FileAr->Tell() < ReplayInfo.DemoFileLastOffset;
}

bool FNullNetworkReplayStreamer::IsLive() const
{
	return IsNamedStreamLive(CurrentStreamName);
}

bool FNullNetworkReplayStreamer::IsNamedStreamLive( const FString& StreamName ) const
{
	// If the final file doesn't exist, this is a live stream.
	return !IFileManager::Get().FileExists(*GetFinalFilename(StreamName));
}

void FNullNetworkReplayStreamer::DeleteFinishedStream(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate)
{
	DeleteFinishedStream(StreamName, Delegate);
}

void FNullNetworkReplayStreamer::DeleteFinishedStream( const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate )
{
	FDeleteFinishedStreamResult Result;

	// Live streams can't be deleted
	if (IsNamedStreamLive(StreamName))
	{
		UE_LOGF(LogNullReplay, Log, "Can't delete network replay stream %ls because it is live!", *StreamName);
	}
	else
	{
		// Delete the directory with the specified name in the Saved/Demos directory
		const FString DemoName = GetStreamDirectory(StreamName);
		if (!FPaths::DirectoryExists(DemoName))
		{
			Result.Result = EStreamingOperationResult::ReplayNotFound;
		}
		else if (IFileManager::Get().DeleteDirectory(*DemoName, false, true))
		{
			Result.Result = EStreamingOperationResult::Success;
		}
	}

	Delegate.ExecuteIfBound(Result);
}

void FNullNetworkReplayStreamer::EnumerateStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate)
{
	// Simply returns a stream for each folder in the Saved/Demos directory
	const FString WildCardPath = ::GetDemoPath() + TEXT( "*" );

	TArray<FString> DirectoryNames;
	IFileManager::Get().FindFiles( DirectoryNames, *WildCardPath, false, true );

	FEnumerateStreamsResult Result;
	
	for ( const FString& Directory : DirectoryNames )
	{
		// Assume there will be one file with a .demo extension in the directory
		const FString FullDemoFilePath = GetDemoFilename( Directory );

		FNetworkReplayStreamInfo Info;
		Info.SizeInBytes = IFileManager::Get().FileSize( *FullDemoFilePath );

		// Read stored info for this replay
		FNullReplayInfo StoredReplayInfo = ReadReplayInfo( Directory );

		if ( !StoredReplayInfo.bIsValid )
		{
			continue;
		}

		// Check version. NetworkVersion and changelist of 0 will ignore version check.
		const bool NetworkVersionMatches = ReplayVersion.NetworkVersion == StoredReplayInfo.NetworkVersion;
		const bool ChangelistMatches = ReplayVersion.Changelist == StoredReplayInfo.Changelist;

		const bool NetworkVersionPasses = ReplayVersion.NetworkVersion == 0 || NetworkVersionMatches;
		const bool ChangelistPasses = ReplayVersion.Changelist == 0 || ChangelistMatches;

		if ( NetworkVersionPasses && ChangelistPasses )
		{
			Info.Name = Directory;
			Info.Timestamp = IFileManager::Get().GetTimeStamp( *FullDemoFilePath );
			Info.bIsLive = IsNamedStreamLive( Directory );
			Info.Changelist = StoredReplayInfo.Changelist;
			Info.LengthInMS = StoredReplayInfo.LengthInMS;
			Info.FriendlyName = StoredReplayInfo.FriendlyName;

			Result.FoundStreams.Add( Info );
		}
	}

	Result.Result = EStreamingOperationResult::Success;
	Delegate.ExecuteIfBound( Result );
}

void FNullNetworkReplayStreamer::AddUserToReplay(const FString& UserString)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::AddUserToReplay is currently unsupported.");
}

void FNullNetworkReplayStreamer::AddEvent( const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data )
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::AddEvent is currently unsupported.");
}

void FNullNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents(Group, Delegate);
}

void FNullNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	EnumerateEvents(Group, Delegate);
}

void FNullNetworkReplayStreamer::EnumerateEvents( const FString& Group, const FEnumerateEventsCallback& Delegate )
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::EnumerateEvents is currently unsupported.");
	FEnumerateEventsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	RequestEventData(EventID, Delegate);
}

void FNullNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	RequestEventData(EventID, Delegate);
}

void FNullNetworkReplayStreamer::RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::RequestEventData is currently unsupported.");
	FRequestEventDataResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::RequestEventGroupData is currently unsupported.");
	FRequestEventGroupDataResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(Group, Delegate);
}

void FNullNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	RequestEventGroupData(Group, Delegate);
}

void FNullNetworkReplayStreamer::SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::SearchEvents is currently unsupported.");
	FSearchEventsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	KeepReplay(ReplayName, bKeep, Delegate);
}

void FNullNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate)
{
	// Replays are kept during streaming so there's no need to explicitly save them.
	// However, sanity check that what was passed in still exists.
	FKeepReplayResult Result;
	if (!FPaths::DirectoryExists(GetStreamDirectory(ReplayName)))
	{
		Result.Result = EStreamingOperationResult::ReplayNotFound;
	}
	else
	{
		Result.Result = EStreamingOperationResult::Success;
		Result.NewReplayName = ReplayName;
	}

	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplayFriendlyName(ReplayName, NewFriendlyName, Delegate);
}

void FNullNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::RenameReplayFriendlyName is currently unsupported.");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplay(ReplayName, NewName, Delegate);
}

void FNullNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::RenameReplay is currently unsupported.");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

void FNullNetworkReplayStreamer::EnumerateRecentStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::EnumerateRecentStreams is currently unsupported.");

	FEnumerateStreamsResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.Execute(Result);
}

FArchive* FNullNetworkReplayStreamer::GetCheckpointArchive()
{
	// If the archive is null, and the API is being used properly, the caller is writing a checkpoint...
	if ( CheckpointAr.Get() == nullptr )
	{
		// Create a file writer for the next checkpoint index.
		check(StreamerState != EReplayStreamerState::Playback);

		FString NextCheckpointFileName = FString::Printf( TEXT( "checkpoint%d" ), CurrentCheckpointIndex );

		UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::GetCheckpointArchive. Creating new checkpoint file.");

		CheckpointAr.Reset( IFileManager::Get().CreateFileWriter( *GetCheckpointFilename(CurrentStreamName, CurrentCheckpointIndex) ) );
	}

	return CheckpointAr.Get();
}

void FNullNetworkReplayStreamer::FlushCheckpoint(const uint32 TimeInMS)
{
	UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::FlushCheckpoint. TimeInMS: %u", TimeInMS);

	check(FileAr.Get() != nullptr);

	// The file writer archive will finalize the file on disk on destruction. The new file will be created
	// next time the driver calls GetCheckpointArchive.
	CheckpointAr.Reset();

	// Also write the event description file to disk with a corresponding checkpoint index, so they can be correlated later.
	TUniquePtr<FArchive> EventFileAr(IFileManager::Get().CreateFileWriter(*GetEventFilename(CurrentStreamName, CurrentCheckpointIndex)));


	if (EventFileAr.Get() != nullptr)
	{
		FNullCheckpointListItem CheckpointEvent;
		CheckpointEvent.Group = TEXT("checkpoint");
		CheckpointEvent.Metadata = FString::Printf( TEXT("%" INT64_FMT), FileAr->Tell() );
		CheckpointEvent.Time1 = TimeInMS;
		CheckpointEvent.Time2 = TimeInMS;

		FString EventJsonString = CheckpointEvent.ToJson();
		*EventFileAr << EventJsonString;
	}

	++CurrentCheckpointIndex;
}

void FNullNetworkReplayStreamer::GotoCheckpointIndex(const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType)
{
	GotoCheckpointIndexInternal(CheckpointIndex, Delegate, -1);
}

void FNullNetworkReplayStreamer::GotoCheckpointIndexInternal(int32 CheckpointIndex, const FGotoCallback& Delegate, int32 ExtraTimeInMS)
{
	check( FileAr.Get() != nullptr);

	FGotoResult Result;
	if ( CheckpointIndex == -1 )
	{
		// Create a dummy checkpoint archive to indicate this is the first checkpoint
		CheckpointAr.Reset(new FArchive);

		FileAr->Seek(0);
		Result.ExtraTimeMS = ExtraTimeInMS;
		Result.Result = EStreamingOperationResult::Success;
		Result.CheckpointInfo.CheckpointIndex = FReplayCheckpointInfo::NO_CHECKPOINT;
		Result.CheckpointInfo.CheckpointStartTime = FReplayCheckpointInfo::NO_CHECKPOINT;
	}
	else
	{
		// Attempt to open the checkpoint file for the given index. Will fail if file doesn't exist.
		const FString CheckpointFilename = GetCheckpointFilename(CurrentStreamName, CheckpointIndex);
		CheckpointAr.Reset(IFileManager::Get().CreateFileReader(*CheckpointFilename));

		if (CheckpointAr.Get() == nullptr)
		{
			UE_LOGF(LogNullReplay, Log, "FNullNetworkReplayStreamer::GotoCheckpointIndex. Index: %i. Couldn't open checkpoint file %ls", CheckpointIndex, *CheckpointFilename);
		}
		else
		{
			Result.ExtraTimeMS = ExtraTimeInMS;
			Result.Result = EStreamingOperationResult::Success;
			Result.CheckpointInfo.CheckpointIndex = CheckpointIndex;

			// Open and deserialize the corresponding event, this tells us where we need to seek to
			// in the main replay file to sync up with the checkpoint we're loading.
			const FString EventFilename = GetEventFilename(CurrentStreamName, CheckpointIndex);
			TUniquePtr<FArchive> EventFile(IFileManager::Get().CreateFileReader(*EventFilename));
			if (EventFile.Get() != nullptr)
			{
				FString JsonString;
				*EventFile << JsonString;

				FNullCheckpointListItem Item;
				Item.FromJson(JsonString);
				Result.CheckpointInfo.CheckpointStartTime = Item.Time1;
				

				// Reopen, since for live replays the file is being written to and read from simultaneously
				// and we need the reported file size to be up to date.
				ReopenStreamFileForReading();

				FileAr->Seek(FCString::Atoi64(*Item.Metadata));
			}
		}
	}

	Delegate.ExecuteIfBound( Result );
}

void FNullNetworkReplayStreamer::ReopenStreamFileForReading()
{
	const FString FullName = GetDemoFilename(CurrentStreamName);
	FileAr.Reset( IFileManager::Get().CreateFileReader( *FullName, FILEREAD_AllowWrite ) );
	if (FileAr.IsValid())
	{
		LastKnownFileSize = FileAr->TotalSize();
	}
}

void FNullNetworkReplayStreamer::UpdateReplayInfoIfValid()
{
	FNullReplayInfo LatestInfo = ReadReplayInfo(CurrentStreamName);

	if (LatestInfo.bIsValid)
	{
		ReplayInfo = LatestInfo;
	}
}

void FNullNetworkReplayStreamer::GotoTimeInMS(const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType)
{
	// Enumerate all the events in the events folder, since we need to know what times the checkpoints correlate with
	TArray<FNullCheckpointListItem> Checkpoints;

	const FString EventBaseName = FPaths::Combine( *GetStreamDirectory(CurrentStreamName), TEXT( "events" ), TEXT( "event" ) );

	int CurrentEventIndex = 0;

	// Try to load every event in order until one is missing
	while ( true )
	{
		const FString CheckEventName = EventBaseName + FString::FromInt(CurrentEventIndex);

		TUniquePtr<FArchive> EventFile( IFileManager::Get().CreateFileReader(*CheckEventName) );

		if ( EventFile.Get() != nullptr )
		{
			FString JsonString;
			*EventFile << JsonString;

			FNullCheckpointListItem Item;
			Item.FromJson(JsonString);

			Checkpoints.Add(Item);
		}
		else
		{
			break;
		}

		CurrentEventIndex++;
	}

	int32 CheckpointIndex = -1;

	if ( Checkpoints.Num() > 0 && TimeInMS >= Checkpoints[ Checkpoints.Num() - 1 ].Time1 )
	{
		// If we're after the very last checkpoint, that's the one we want
		CheckpointIndex = Checkpoints.Num() - 1;
	}
	else
	{
		// Checkpoints should be sorted by time, return the checkpoint that exists right before the current time
		// For fine scrubbing, we'll fast forward the rest of the way
		// NOTE - If we're right before the very first checkpoint, we'll return -1, which is what we want when we want to start from the very beginning
		for ( int i = 0; i < Checkpoints.Num(); i++ )
		{
			if ( TimeInMS < Checkpoints[i].Time1 )
			{
				CheckpointIndex = i - 1;
				break;
			}
		}
	}

	int32 ExtraSkipTimeInMS = TimeInMS;

	if ( CheckpointIndex >= 0 )
	{
		// Subtract off checkpoint time so we pass in the leftover to the engine to fast forward through for the fine scrubbing part
		ExtraSkipTimeInMS = TimeInMS - Checkpoints[ CheckpointIndex ].Time1;
	}

	GotoCheckpointIndexInternal( CheckpointIndex, Delegate, ExtraSkipTimeInMS );
}

void FNullNetworkReplayStreamer::Tick(float DeltaSeconds)
{
	// This relies on the fact that the DemoNetDriver isn't currently in the middle of its own tick,
	// and has either read or written a whole demo frame.
	if (StreamerState == EReplayStreamerState::Playback)
	{
		// Re-read replay info
		UpdateReplayInfoIfValid();

		// If there are new whole frames to read in the file, reopen it to refresh the size
		if (ReplayInfo.DemoFileLastOffset > LastKnownFileSize)
		{
			const int64 OldLocation = FileAr->Tell();
			ReopenStreamFileForReading();
			if (FileAr.IsValid())
			{
				FileAr->Seek(OldLocation);
			}
		}
	}
	else if (StreamerState == EReplayStreamerState::Recording)
	{
		// Note the size of the file between demo frames
		if (FileAr.IsValid() && ReplayInfo.DemoFileLastOffset < FileAr->Tell())
		{
			ReplayInfo.DemoFileLastOffset = IntCastChecked<int32>(FileAr->Tell());
			FileAr->Flush();

			WriteReplayInfo(CurrentStreamName, ReplayInfo);
		}
	}
}

TStatId FNullNetworkReplayStreamer::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNullNetworkReplayStreamer, STATGROUP_Tickables);
}

EStreamingOperationResult FNullNetworkReplayStreamer::GetDemoPath(FString& DemoPath) const
{
	DemoPath = ::GetDemoPath();
	return EStreamingOperationResult::Success;
}

TSharedPtr< INetworkReplayStreamer > FNullNetworkReplayStreamingFactory::CreateReplayStreamer() 
{
	return TSharedPtr< INetworkReplayStreamer >( new FNullNetworkReplayStreamer );
}
