// Copyright Epic Games, Inc. All Rights Reserved.

#include "NullNetworkReplayStreaming.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FNullNetworkReplayStreamingFactory, NullNetworkReplayStreaming)
