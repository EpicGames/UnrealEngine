// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Serialization/JsonSerializerMacros.h"
#include "NetworkReplayStreaming.h"
#include "Tickable.h"

class FNetworkReplayVersion;

/* Class to hold metadata about an entire replay */
class FNullReplayInfo : public FJsonSerializable
{
public:
	FNullReplayInfo() : LengthInMS(0), NetworkVersion(0), Changelist(0), DemoFileLastOffset(0), bIsValid(false) {}

	int32		LengthInMS;
	uint32		NetworkVersion;
	uint32		Changelist;
	FString		FriendlyName;
	int32		DemoFileLastOffset;
	bool		bIsValid;

	// FJsonSerializable
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE( "LengthInMS",		LengthInMS );
		JSON_SERIALIZE( "NetworkVersion",	NetworkVersion );
		JSON_SERIALIZE( "Changelist",		Changelist );
		JSON_SERIALIZE( "FriendlyName",		FriendlyName );
		JSON_SERIALIZE( "DemoFileLastOffset",	DemoFileLastOffset );
	END_JSON_SERIALIZER
};

/** Default streamer that goes straight to the HD */
class FNullNetworkReplayStreamer : public INetworkReplayStreamer, public FTickableGameObject
{
public:
	FNullNetworkReplayStreamer() :
		StreamerState(EReplayStreamerState::Idle),
		CurrentCheckpointIndex( 0 ),
		LastKnownFileSize( 0 )
	{}
	
	/** INetworkReplayStreamer implementation */
	virtual void StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate) override;
	virtual void StopStreaming() override;
	virtual FArchive* GetHeaderArchive() override;
	virtual FArchive* GetStreamingArchive() override;
	virtual FArchive* GetCheckpointArchive() override;
	virtual void FlushCheckpoint( const uint32 TimeInMS ) override;
	virtual void GotoCheckpointIndex( const int32 CheckpointIndex, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType ) override;
	virtual void GotoTimeInMS( const uint32 TimeInMS, const FGotoCallback& Delegate, EReplayCheckpointType CheckpointType ) override;
	virtual void UpdateTotalDemoTime( uint32 TimeInMS ) override;
	virtual void UpdatePlaybackTime(uint32 TimeInMS) override {}
	virtual uint32 GetTotalDemoTime() const override { return ReplayInfo.LengthInMS; }
	virtual bool IsDataAvailable() const override;
	virtual void SetHighPriorityTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS ) override { }
	virtual bool IsDataAvailableForTimeRange( const uint32 StartTimeInMS, const uint32 EndTimeInMS ) override { return true; }
	virtual bool IsLoadingCheckpoint() const override { return false; }
	virtual bool IsLive() const override;
	virtual void DeleteFinishedStream( const FString& StreamName, const FDeleteFinishedStreamCallback& Delegate) override;
	virtual void DeleteFinishedStream( const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate ) override;
	virtual void EnumerateStreams( const FNetworkReplayVersion& InReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate ) override;
	virtual void EnumerateRecentStreams( const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate ) override;
	virtual void AddUserToReplay(const FString& UserString) override;
	virtual void AddEvent(const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data) override;
	virtual void AddOrUpdateEvent( const FString& Name, const uint32 TimeInMS, const FString& Group, const FString& Meta, const TArray<uint8>& Data ) override {}

	virtual void EnumerateEvents( const FString& Group, const FEnumerateEventsCallback& EnumerationCompleteDelegate ) override;
	virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& EnumerationCompleteDelegate ) override;
	virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate ) override;

	virtual void RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	virtual void RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	virtual void RequestEventData(const FString& ReplayName, const FString& EventId, const int32 UserIndex, const FRequestEventDataCallback& Delegate) override;
	virtual void RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate) override;
	virtual void SearchEvents(const FString& EventGroup, const FSearchEventsCallback& Delegate) override;
	virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate) override;
	virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate) override;
	virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate) override;
	virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate) override;
	virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	virtual FString	GetReplayID() const override { return CurrentStreamName; }
	virtual EReplayStreamerState GetReplayStreamerState() const override { return StreamerState; }
	virtual void SetTimeBufferHintSeconds(const float InTimeBufferHintSeconds) override {}
	virtual void RefreshHeader() override {};
	virtual void DownloadHeader(const FDownloadHeaderCallback& Delegate) override
	{
		FDownloadHeaderResult Result;
		Result.Result = EStreamingOperationResult::Success;
		Delegate.Execute(Result);
	}

	/** FTickableObjectBase implementation */
	virtual void Tick(float DeltaSeconds) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;

	/** FTickableGameObject implementation */
	virtual bool IsTickableWhenPaused() const override { return true; }

	virtual uint32 GetMaxFriendlyNameSize() const override { return 0; }

	virtual EStreamingOperationResult SetDemoPath(const FString& DemoPath) override
	{
		return EStreamingOperationResult::Unsupported;
	}

	virtual EStreamingOperationResult GetDemoPath(FString& DemoPath) const override;

	virtual bool IsCheckpointTypeSupported(EReplayCheckpointType CheckpointType) const override { return (CheckpointType == EReplayCheckpointType::Full); }

private:
	bool IsNamedStreamLive( const FString& StreamName ) const;

	/** Handles the details of loading a checkpoint */
	void GotoCheckpointIndexInternal(int32 CheckpointIndex, const FGotoCallback& Delegate, int32 TimeInMS);

	/** Reopen the file to refresh it's size, since file-based FArchives do not appear to update their size if they're being written to. */
	void ReopenStreamFileForReading();

	/** Overwrites the cached ReplayInfo only if the read succeeded */
	void UpdateReplayInfoIfValid();

	/** Handle to the archive that will read/write the demo header */
	TUniquePtr<FArchive> HeaderAr;

	/** Handle to the archive that will read/write network packets */
	TUniquePtr<FArchive> FileAr;

	/* Handle to the archive that will read/write checkpoint files */
	TUniquePtr<FArchive> CheckpointAr;

	/** Overall state of the streamer */
	EReplayStreamerState StreamerState;

	/** Remember the name of the current stream, if any. */
	FString CurrentStreamName;

	/** Current number of checkpoints written. */
	int32 CurrentCheckpointIndex;

	/** Currently playing or recording replay metadata */
	FNullReplayInfo ReplayInfo;

	/** Last known size of the replay stream file. */
	int64 LastKnownFileSize;
};

class FNullNetworkReplayStreamingFactory : public INetworkReplayStreamingFactory
{
public:
	virtual TSharedPtr< INetworkReplayStreamer > CreateReplayStreamer();
};
