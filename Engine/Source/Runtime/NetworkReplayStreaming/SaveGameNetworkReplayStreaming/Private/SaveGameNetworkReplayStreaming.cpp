// Copyright Epic Games, Inc. All Rights Reserved.

#include "SaveGameNetworkReplayStreaming.h"
#include "Misc/Paths.h"
#include "PlatformFeatures.h"
#include "SaveGameSystem.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Logging/LogMacros.h"
#include "HAL/IConsoleManager.h"
#include "Async/Async.h"
#include "Tickable.h"
#include "GameDelegates.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "GameFramework/PlayerController.h"

DEFINE_LOG_CATEGORY_STATIC(LogSaveGameReplay, Log, All);

#ifndef PLATFORM_MOVE_REQUIRES_LOWERCASE
#define PLATFORM_MOVE_REQUIRES_LOWERCASE 0
#endif // PLATFORM_MOVE_REQUIRES_LOWERCASE

TAutoConsoleVariable<FString> CVarSaveGameFilterEventGroup(
	TEXT("demo.SaveGameEventFilter"),
	FString(),
	TEXT("When set to a non-empty string, only replay events in the specified group will be saved to header meta-data.")
);

static const FString SaveReplayExt(".sav_rep");
#define ReplaySaveFileFormat TEXT("rep_%d.sav_rep")
#define ReplayMetaSaveFileFormat TEXT("repmet_%d.sav_rep")

namespace SaveGameReplay
{
	enum ESaveGameHeaderVersionHistory : uint32
	{
		HISTORY_INITIAL = 0,
		HISTORY_EVENTS = 1,

		// -----<new versions can be added before this line>-------------------------------------------------
		HISTORY_PLUS_ONE,
		HISTORY_LATEST = HISTORY_PLUS_ONE - 1
	};

	typedef FSaveGameNetworkReplayStreamer FStreamer;

	template<typename TResult>
	struct TAsyncTypes
	{
		typedef TSharedPtr<TResult, ESPMode::ThreadSafe> TSharedResult;

		typedef TFunction<TSharedResult()> TAsyncFunc;
		typedef TFunction<void(const TResult& Result)> TPostAsyncFunc;

		static TSharedResult MakeSharedResult()
		{
			// Can't directly use MakeShared here because there's no version that taks ESPMode properly.
			return TSharedResult(new TResult());
		}
	};

	bool IsSaveGameFileName(const FString& ReplayName)
	{
		return ReplayName.EndsWith(SaveReplayExt);
	}

	int32 GetReplayIndexFromName(const FString& ReplayName)
	{
		// Validate it's an appropriate save name and Grab the replay index.
		// The replay index should be immediately before the replay extension, and immediately after the last underscore
		// in the replay name. So, we can inspect that part of the replay and convert it to an int to determine the index.
		if (ensureMsgf(IsSaveGameFileName(ReplayName), TEXT("GetReplayIndexFromName called with non-save name %s"), *ReplayName))
		{
			const int32 EndIndexPos = ReplayName.Len() - SaveReplayExt.Len();
			if (EndIndexPos > 0)
			{
				const int32 StartIndexPos = ReplayName.Find(TEXT("_"), ESearchCase::CaseSensitive, ESearchDir::FromEnd, EndIndexPos) + 1;
				if (StartIndexPos > 0 && EndIndexPos > StartIndexPos)
				{
					int32 ReplayIndex = 0;
					if (LexTryParseString(ReplayIndex, *ReplayName.Mid(StartIndexPos, EndIndexPos - StartIndexPos)))
					{
						return ReplayIndex;
					}
				}
			}
		}

		return INDEX_NONE;
	}

	class FAsyncTaskManager : private FTickableGameObject
	{
	private:

		class FAsyncTaskBase
		{
		public:

			FAsyncTaskBase(const FStreamer& OwningStreamer, const FString& InDescription) :
				StreamerSharedRef(OwningStreamer.AsShared()),
				Description(InDescription)
			{
			}

			virtual ~FAsyncTaskBase() {}

			virtual bool HasFinished() const = 0;

			virtual void Finalize() = 0;

			const FString& GetDescription()
			{
				return Description;
			}

		private:

			// We hold onto a reference of the streamer to make sure it stays alive long enough
			// to complete this task.
			const TSharedRef<const FLocalFileNetworkReplayStreamer> StreamerSharedRef;
			const FString Description;
		};

		template<typename TResult>
		class TAsyncTask : public FAsyncTaskBase
		{
		public:

			typedef typename TAsyncTypes<TResult>::TSharedResult TSharedResult;
			typedef typename TAsyncTypes<TResult>::TAsyncFunc TAsyncFunc;
			typedef typename TAsyncTypes<TResult>::TPostAsyncFunc TPostAsyncFunc;

			TAsyncTask(const FStreamer& OwningStreamer, const FString& Description, TAsyncFunc InAsyncWork, TPostAsyncFunc InPostAsyncWork) :
				FAsyncTaskBase(OwningStreamer, Description),
				Future(Async(EAsyncExecution::Thread, InAsyncWork)),
				PostAsyncWork(InPostAsyncWork)
			{
			}

			virtual bool HasFinished() const
			{
				return Future.IsReady();
			}

			virtual void Finalize() override
			{
				PostAsyncWork(*Future.Get().Get());
			}

			TFuture<TSharedResult> Future;
			TPostAsyncFunc PostAsyncWork;
		};

	public:

		static FAsyncTaskManager& Get()
		{
			static FAsyncTaskManager TaskManager;
			return TaskManager;
		}

		template<typename TResult>
		void StartTask(const FStreamer& OwningStreamer, const FString& Description, typename TAsyncTypes<TResult>::TAsyncFunc InAsyncWork, typename TAsyncTypes<TResult>::TPostAsyncFunc InPostAsyncWork)
		{
			if (ensureMsgf(IsInGameThread(), TEXT("SaveGameReplay::FAsyncTaskManager::StartTask - Called from outside the GameThread.")))
			{
				if (AreAnyTasksOutstanding())
				{
					UE_LOGF(LogSaveGameReplay, Warning, "SaveGameReplay::FAsyncTaskManager::StartTask - New task attempted while processing pending task (NewTask = %ls PendingTask = %ls)", *Description, *(OutstandingTask->GetDescription()));

					TResult Result;
					Result.Result = EStreamingOperationResult::UnfinishedTask;
					InPostAsyncWork(Result);
				}
				else
				{
					if (InAsyncWork)
					{
						OutstandingTask = MakeUnique<TAsyncTask<TResult>>(OwningStreamer, Description, InAsyncWork, InPostAsyncWork);
					}
					else
					{
						UE_LOGF(LogSaveGameReplay, Warning, "SaveGameReplay::FAsyncTaskManager::StartTask - Async work function was invalid (NewTask = %ls)", *Description);

						TResult Result;
						Result.Result = EStreamingOperationResult::UnfinishedTask;
						InPostAsyncWork(Result);
					}
				}
			}
		}

		const bool AreAnyTasksOutstanding() const
		{
			return OutstandingTask.IsValid();
		}

	private:

		FAsyncTaskManager() {}
		~FAsyncTaskManager() {}
		FAsyncTaskManager(const FAsyncTaskManager&) = delete;

		virtual void Tick(float DeltaTime) override
		{
			if (OutstandingTask.IsValid() && OutstandingTask->HasFinished())
			{
				// Transfer ownership so the OutstandingTask is cleared, but still temporarily alive.
				// This will allow us to start new tasks / check state appropriately from Finalize.
				TUniquePtr<FAsyncTaskBase> LocalTask(OutstandingTask.Release());
				LocalTask->Finalize();
			}
		}

		virtual bool IsTickable() const override
		{
			return OutstandingTask.IsValid();
		}

		/** return the stat id to use for this tickable **/
		virtual TStatId GetStatId() const override
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(SaveGameReplayAsyncTaskManager, STATGROUP_Tickables);
		}

		TUniquePtr<FAsyncTaskBase> OutstandingTask;
	};

	class FSaveGameReplayMoveFileHelper
	{
	private:

		FSaveGameReplayMoveFileHelper() = delete;
		~FSaveGameReplayMoveFileHelper() = delete;

		struct FMoveContext
		{
			FMoveContext(const TSharedPtr<INetworkReplayStreamer>& InStreamer, const FString& InSourceDirectory, const FString& InDestinationDirectory, const int32 InUserIndex) :
				Streamer(InStreamer),
				SourceDirectory(InSourceDirectory),
				DestinationDirectory(InDestinationDirectory),
				UserIndex(InUserIndex)
			{
			}

			const TSharedPtr<INetworkReplayStreamer> Streamer;
			const FString SourceDirectory;
			const FString DestinationDirectory;

			// It's usually not safe to cache user indices, but this is a development only feature so it's probably OK.
			// If this gets co-opted for non-dev stuff at some point, maybe consider a weak ptr to the ULocalPlayer that issued the request.
			const int32 UserIndex;
		};

		static void RunCommand(const TArray<FString>& Params, const TFunction<void(const TSharedPtr<INetworkReplayStreamer>&, const FString&, const int32)> CommandToRun)
		{
			if (IsRunningDedicatedServer())
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper commands are client only.");
				return;
			}

			const TCHAR* StreamerOverride = nullptr;
			if (Params.Num() == 1)
			{
				StreamerOverride = *Params[0];
			}
			else if (Params.Num() != 0)
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper commands take either a Streamer Override or no arguments.");
				return;
			}

			// Note, RTTI is disabled by default so there's no way to tell what the actual streamer type is (without embedding it ourselves).
			// Therefore, just assume that if the streamer is valid and supports local file operations, that this will work.
			// This should be fine, because the Console Commands shouldn't be available unless the SaveGameStreamer module is linked anyway.
			// (Note, we could move the commands to Module startup / shutdown which would guarantee that the module was actually loaded, and not just linked).
			TSharedPtr<INetworkReplayStreamer> Streamer = FNetworkReplayStreaming::Get().GetFactory(StreamerOverride).CreateReplayStreamer();

			if (ensureMsgf(Streamer.IsValid(), TEXT("FSaveGameReplayMoveFileHelper Invalid local streamer")))
			{
				FString DemoPath;
				if (ensureMsgf(EStreamingOperationResult::Success == Streamer->GetDemoPath(DemoPath), TEXT("FSaveGameReplayMoveFileHelper Streamer not supported.")))
				{
					const int32 UserIndex = GetFirstPlayerIndex();
					if (ensureMsgf(INDEX_NONE != UserIndex, TEXT("FSaveGameReplayMoveFileHelper Unable to get UserIndex")))
					{
						CommandToRun(Streamer, DemoPath, UserIndex);
					}
				}
			}
		}

		static int32 GetFirstPlayerIndex()
		{
			if (GEngine)
			{
				if (UWorld* World = GWorld.GetReference())
				{
					if (APlayerController* Controller = GEngine->GetFirstLocalPlayerController(World))
					{
						if (ULocalPlayer* Player = Controller->GetLocalPlayer())
						{
							return Player->GetControllerId();
						}
					}
				}
			}

			return INDEX_NONE;
		}

		static void SanitizeUnsavedNames(const TArray<FString>& Params)
		{
			RunCommand(Params, [](const TSharedPtr<INetworkReplayStreamer>& Streamer, const FString& DemoPath, const int32 UserIndex)
			{
				FSaveGameReplayMoveFileHelper::SanitizeNames(DemoPath);
			});
		}

		static void ImportReplayFiles(const TArray<FString>& Params)
		{
			RunCommand(Params, [](const TSharedPtr<INetworkReplayStreamer>& Streamer, const FString& DemoPath, const int32 UserIndex)
			{
				FSaveGameReplayMoveFileHelper::MoveFilesFromTemp(DemoPath, UserIndex);
			});
		}

		static void ExportReplayFiles(const TArray<FString>& Params)
		{
			RunCommand(Params, [](const TSharedPtr<INetworkReplayStreamer>& Streamer, const FString& DemoPath, const int32 UserIndex)
			{
				FSaveGameReplayMoveFileHelper::MoveFilesToTemp(Streamer, DemoPath, UserIndex);
			});
		}

		static void SanitizeNames(const FString& DemoPath)
		{
			IFileManager& FileManager = IFileManager::Get();

			const FString LocalFileExt = FNetworkReplayStreaming::GetReplayFileExtension();
			const FString WildCard(FPaths::Combine(DemoPath, FString::Printf(TEXT("*%s%s"), *SaveReplayExt, *LocalFileExt)));
			TArray<FString> FoundFiles;

			FileManager.FindFiles(FoundFiles, *WildCard, /*bFiles=*/true, /*bDirectories=*/false);

			for (const FString& CurrentName : FoundFiles)
			{
				UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::SanitizeNames - Handling %ls", *CurrentName);

				FString NewName(CurrentName);
				NewName.RemoveFromEnd(LocalFileExt);
				MakeUniqueReplayName(NewName);

				if (!FileManager.Move(*FPaths::Combine(DemoPath, NewName), *FPaths::Combine(DemoPath, CurrentName)))
				{
					UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::SanitizeNames - Failed to sanitize %ls", *CurrentName);
				}
			}
		}

		static void MakeUniqueReplayName(FString& Name)
		{
			IFileManager& FileManager = IFileManager::Get();

			// Make sure to sanitize this so the system doesn't get tricked into thinking this is a saved replay.
			// Note, this may still happen if the usually manually entered this...
			Name.RemoveFromEnd(SaveReplayExt);
			
			const FString LocalFileExt = FNetworkReplayStreaming::GetReplayFileExtension();

			int32 Index = 1;
			FString UseName = FString::Printf(TEXT("%s%s"), *Name, *LocalFileExt);
			while (FileManager.FileExists(*UseName))
			{
				UseName = FString::Printf(TEXT("%s - %d%s"), *Name, ++Index, *LocalFileExt);
			}

			Name = MoveTemp(UseName);
		}

		static void MoveFilesInternal(const TSharedPtr<FMoveContext>& Context)
		{
			Context->Streamer->EnumerateStreams(FNetworkReplayVersion(), Context->UserIndex, FString(), TArray<FString>(), FEnumerateStreamsCallback::CreateStatic(FSaveGameReplayMoveFileHelper::OnEnumerateStreamsComplete, Context));
		}

		static void MoveFilesInternal_PostEnumerate(const TSharedPtr<FMoveContext>& Context)
		{
			Context->Streamer->EnumerateRecentStreams(FNetworkReplayVersion(), Context->UserIndex, FEnumerateStreamsCallback::CreateStatic(FSaveGameReplayMoveFileHelper::OnEnumerateRecentStreamsComplete, Context));
		}

		static void CopyFile(const TSharedPtr<FMoveContext>& Context, const FString& BaseFileName)
		{
			const FString LocalFileExt = FNetworkReplayStreaming::GetReplayFileExtension();
			const FString SourceFileName = FPaths::Combine(Context->SourceDirectory, BaseFileName) + LocalFileExt;

			FString DestinationFileName = FPaths::Combine(Context->DestinationDirectory, BaseFileName);
			MakeUniqueReplayName(DestinationFileName);

			const uint32 Result = IFileManager::Get().Copy(*DestinationFileName, *SourceFileName);
			if (0 != Result)
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::CopyFile: Failed - from '%ls' to '%ls' error = %lu", *SourceFileName, *DestinationFileName, Result);
			}

			UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::CopyFile - from '%ls' to '%ls': Result = %d", *SourceFileName, *DestinationFileName, Result);
		}

		static void SaveFile(const TSharedPtr<FMoveContext>& Context, const FString& SaveGameName, ISaveGameSystem* SaveGameSystem)
		{
			TArray<uint8> SaveData;
			if (!SaveGameSystem->LoadGame(false, *SaveGameName, Context->UserIndex, SaveData))
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::SaveFile: Failed to load save game %ls", *SaveGameName)
					return;
			}

			FString DestinationFileName = FPaths::Combine(Context->DestinationDirectory, SaveGameName);
			MakeUniqueReplayName(DestinationFileName);

			TUniquePtr<FArchive> FileAR(IFileManager::Get().CreateFileWriter(*DestinationFileName));
			FileAR->Serialize(SaveData.GetData(), SaveData.Num());

			if (FileAR->IsError())
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::SaveFile: Failed to save game %ls to %ls", *SaveGameName, *DestinationFileName);
			}
		}

		static void OnEnumerateStreamsComplete(const FEnumerateStreamsResult& Result, const TSharedPtr<FMoveContext> Context)
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::OnEnumerateStreamsComplete: Success=%ls NumFiles=%d", *(Result.WasSuccessful() ? CoreTexts.True : CoreTexts.False).ToString(), Result.FoundStreams.Num());

			if (Result.WasSuccessful())
			{
				if (Result.FoundStreams.Num() > 0)
				{
					if (IsSaveGameFileName(Result.FoundStreams[0].Name))
					{
						if (ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem())
						{
							for (const FNetworkReplayStreamInfo& StreamInfo : Result.FoundStreams)
							{
								SaveFile(Context, StreamInfo.Name, SaveGameSystem);
							}
						}
						else
						{
							UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::OnEnumerateRecentStreamsComplete: Unable to get SaveGameSystem");
						}
					}
					else
					{
						Async(EAsyncExecution::TaskGraph,
							[Result, Context]()
							{
								for (const FNetworkReplayStreamInfo& StreamInfo : Result.FoundStreams)
								{
									CopyFile(Context, StreamInfo.Name);
								}
							},
							[Context]()
							{
								UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::OnEnumerateStreamsComplete: Async copy complete.");

								AsyncTask(ENamedThreads::GameThread, 
									[Context]()
									{
										MoveFilesInternal_PostEnumerate(Context);
									});
							});

						return;
					}
				}
			}
			else
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::OnEnumerateRecentStreamsComplete: Enumerate failed");
			}

			MoveFilesInternal_PostEnumerate(Context);
		}

		static void OnEnumerateRecentStreamsComplete(const FEnumerateStreamsResult& Result, const TSharedPtr<FMoveContext> Context)
		{
			if (Result.WasSuccessful())
			{
				Async(EAsyncExecution::TaskGraph, 
					[Result, Context]()
					{
						// Currently, the LocalFileStreamer doesn't support EnumerateRecentStreams
						// and the SaveGameStreamer will just return non-Saved replays.
						for (const FNetworkReplayStreamInfo& StreamInfo : Result.FoundStreams)
						{
							CopyFile(Context, StreamInfo.Name);
						}
					},
					[]()
					{
						UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::OnEnumerateRecentStreamsComplete: Async copy complete.")
					});
			}
			else
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::OnEnumerateRecentStreamsComplete: Enumerate failed")
			}
		}

		static FString& GetTempDemoDirectory()
		{
			static FString TempDemoDir = FPaths::Combine(FPaths::ProjectLogDir(), TEXT("Demos/"));
			return TempDemoDir;
		}

		static void MoveFilesLocalInternal(const TSharedPtr<FMoveContext>& MoveContext)
		{
			const FString LocalFileExt = FNetworkReplayStreaming::GetReplayFileExtension();

			TArray<FString> ReplayFiles;
			IFileManager::Get().FindFiles(ReplayFiles, *MoveContext->SourceDirectory, *LocalFileExt);

			for (FString& ReplayFileName : ReplayFiles)
			{
				ReplayFileName.RemoveFromEnd(LocalFileExt);

#if PLATFORM_MOVE_REQUIRES_LOWERCASE
				if (ReplayFileName.Compare(ReplayFileName.ToLower(), ESearchCase::CaseSensitive) != 0)
				{
					UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameReplayMoveFileHelper::MoveFilesLocalInternal - Replay file %ls is not lowercase, import will fail.", *ReplayFileName);
				}
#endif

				CopyFile(MoveContext, ReplayFileName);
			}
		}

	public:

		static void MoveFiles(const TSharedPtr<INetworkReplayStreamer>& Streamer, const FString& DestinationDirectory, const FString& SourceDirectory, const int32 UserIndex)
		{
			UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::MoveFiles: Moving files from %ls to %ls", *FPaths::ConvertRelativePathToFull(SourceDirectory), *FPaths::ConvertRelativePathToFull(DestinationDirectory));

			MoveFilesInternal(MakeShareable(new FMoveContext(Streamer, SourceDirectory, DestinationDirectory, UserIndex)));
		}

		static void MoveFilesToTemp(const TSharedPtr<INetworkReplayStreamer>& Streamer, const FString& SourceDirectory, const int32 UserIndex)
		{
			MoveFiles(Streamer, GetTempDemoDirectory(), SourceDirectory, UserIndex);
		}

		static void MoveFilesFromTemp(const FString& DestinationDirectory, const int32 UserIndex)
		{
			TSharedPtr<FMoveContext> MoveContext(MakeShareable(new FMoveContext(nullptr, GetTempDemoDirectory(), DestinationDirectory, UserIndex)));

			Async(EAsyncExecution::TaskGraph,
				[MoveContext]()
				{
					MoveFilesLocalInternal(MoveContext);
				},
				[]()
				{
					UE_LOGF(LogSaveGameReplay, Log, "FSaveGameReplayMoveFileHelper::MoveFilesFromTemp: Async copy complete.")
				});
		}

	private:

		static FAutoConsoleCommand ImportReplayFilesCommand;
		static FAutoConsoleCommand ExportReplayFilesCommand;
		static FAutoConsoleCommand SanitizedUnsavedNamesCommand;
	};

	FAutoConsoleCommand FSaveGameReplayMoveFileHelper::ImportReplayFilesCommand(
		TEXT("SaveGameStreamerImportReplays"),
		TEXT("Imports replays from the default demo path in FLocalFileNetworkReplayStreamer into the default demo path from FSaveGameNetworkReplayStreamer."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&FSaveGameReplayMoveFileHelper::ImportReplayFiles)
	);

	FAutoConsoleCommand FSaveGameReplayMoveFileHelper::ExportReplayFilesCommand(
		TEXT("SaveGameStreamerExportReplays"),
		TEXT("Exports replays from both the default demo path and saved demo path in FSaveGameNetworkReplayStreamer and copies them to the default demo path in FSaveGameNetworkReplayStreamer."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&FSaveGameReplayMoveFileHelper::ExportReplayFiles)
	);

	FAutoConsoleCommand FSaveGameReplayMoveFileHelper::SanitizedUnsavedNamesCommand(
		TEXT("SaveGameStreamerSanitizedUnsavedNames"),
		TEXT("Removes the 'saved replay' postfix from any unsaved replays. This can be used to fix issues where saved replays become unusable after exporting and reimporting."),
		FConsoleCommandWithArgsDelegate::CreateStatic(&FSaveGameReplayMoveFileHelper::SanitizeUnsavedNames)
	);

	struct FScopedBindExtendedSaveDelegate
	{
		typedef TFunction<bool(const TCHAR*, EGameDelegates_SaveGame, FString&)> FDelegateFunction;

		FScopedBindExtendedSaveDelegate(FDelegateFunction Func) :
			Function(Func)
		{
			FExtendedSaveGameInfoDelegate& ExtendedSaveGameInfoDelegate = FGameDelegates::Get().GetExtendedSaveGameInfoDelegate();

			if (ExtendedSaveGameInfoDelegate.IsBound())
			{
				OldDelegate = ExtendedSaveGameInfoDelegate;
			}

			ExtendedSaveGameInfoDelegate.BindRaw(this, &FScopedBindExtendedSaveDelegate::Execute);
			Handle = ExtendedSaveGameInfoDelegate.GetHandle();
		}

		~FScopedBindExtendedSaveDelegate()
		{
			FExtendedSaveGameInfoDelegate& ExtendedSaveGameInfoDelegate = FGameDelegates::Get().GetExtendedSaveGameInfoDelegate();
			
			if (ensureMsgf(Handle == ExtendedSaveGameInfoDelegate.GetHandle(), TEXT("FScopedBindExtendedSaveDelegate: Delegate binding was changed within scope lifecycle.")) &&
				OldDelegate.IsSet() && OldDelegate->IsBound())
			{
				ExtendedSaveGameInfoDelegate = OldDelegate.GetValue();
			}
			else
			{
				ExtendedSaveGameInfoDelegate.Unbind();
			}
		}

		void Execute(const TCHAR* FileName, EGameDelegates_SaveGame Option, FString& OptionValue)
		{
			if (!Function(FileName, Option, OptionValue) && OldDelegate.IsSet())
			{
				OldDelegate->ExecuteIfBound(FileName, Option, OptionValue);
			}
		}

	private:

		FScopedBindExtendedSaveDelegate(const FScopedBindExtendedSaveDelegate&) = delete;
		FScopedBindExtendedSaveDelegate(FScopedBindExtendedSaveDelegate&&) = delete;

		FDelegateFunction Function;
		TOptional<FExtendedSaveGameInfoDelegate> OldDelegate;
		FDelegateHandle Handle;
	};

	int32 MaxEvents = 32 * 1024;
	static FAutoConsoleVariableRef CVarMaxEvents(TEXT("SaveGameReplay.MaxEvents"), MaxEvents, TEXT("Maximum number of expected events per replay."));

	int32 MaxEventSize = 128 * 1024;
	static FAutoConsoleVariableRef CVarMaxEventSize(TEXT("SaveGameReplay.MaxEventSize"), MaxEventSize, TEXT("Maximum expected size of a replay event in bytes.."));
}

struct FConstConsoleVars
{
	static const int32 GetMaxNumReplaySlots()
	{
		return MaxNumReplaySlots;
	}

private:

	static int32 MaxNumReplaySlots;
	static FAutoConsoleVariableRef CVarMaxNumReplaySlots;
};

int32 FConstConsoleVars::MaxNumReplaySlots = 10;
FAutoConsoleVariableRef FConstConsoleVars::CVarMaxNumReplaySlots( TEXT("demo.MaxNumReplaySlots"), FConstConsoleVars::MaxNumReplaySlots, TEXT("Maximum number of save slots to consider when using the FSaveGameNetworkReplayStreamer"));

static void PopulateStreamingResultFromSaveExistsResult(const ISaveGameSystem::ESaveExistsResult SaveExistsResult, FStreamingResultBase& StreamingResult)
{
	using ESaveExistsResult = ISaveGameSystem::ESaveExistsResult;

	if (ESaveExistsResult::Corrupt == SaveExistsResult)
	{
		StreamingResult.Result = EStreamingOperationResult::ReplayCorrupt;
	}
	else if (ESaveExistsResult::DoesNotExist == SaveExistsResult)
	{
		StreamingResult.Result = EStreamingOperationResult::ReplayNotFound;
	}
	else if (ESaveExistsResult::OK == SaveExistsResult)
	{
		StreamingResult.Result = EStreamingOperationResult::Success;
	}
	else
	{
		StreamingResult.Result = EStreamingOperationResult::Unspecified;
	}
}

FSaveGameNetworkReplayStreamer::FSaveGameNetworkReplayStreamer() :
	FLocalFileNetworkReplayStreamer(GetDefaultDemoSavePath()),
	PlaybackReplayName(GetDefaultPlaybackName())
{
}

FSaveGameNetworkReplayStreamer::FSaveGameNetworkReplayStreamer(const FString& InDemoPath, const FString& InPlaybackReplayName) :
	FLocalFileNetworkReplayStreamer(InDemoPath),
	PlaybackReplayName(ensureMsgf(!InPlaybackReplayName.IsEmpty(), TEXT("FSaveGameNetworkReplayStreamer: InPlaybackReplayName was empty, using default.")) ? InPlaybackReplayName : GetDefaultPlaybackName())
{
}

void FSaveGameNetworkReplayStreamer::StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate)
{
	// If we're not handling a SaveFile directly, then just do normal streaming behavior.
	if (!IsSaveGameFileName(Params.CustomName))
	{
		FLocalFileNetworkReplayStreamer::StartStreaming(Params, Delegate);
	}
	else if (Params.UserIndices.Num() > 0)
	{
		StartStreamingSaved(Params, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::StartStreaming - Invalid UserString");
		Delegate.ExecuteIfBound(FStartStreamingResult());
	}
}

void FSaveGameNetworkReplayStreamer::StartStreamingSaved(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate)
{
	// We should only hit this path if we're playing back a replay.
	check(!Params.bRecord);

	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FStartStreamingResult>;

	auto AsyncWork = [this, Params]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		StartStreaming_Internal(Params, *SharedResult.Get());
		return SharedResult;
	};

	auto PostAsyncWork = [this, Params, Delegate](const FStartStreamingResult& Result)
	{
		if (Result.WasSuccessful())
		{
			FStartStreamingParameters LocalParams = Params;
			LocalParams.CustomName = GetLocalPlaybackName();
			LocalParams.FriendlyName = FString();

			FLocalFileNetworkReplayStreamer::StartStreaming(LocalParams, Delegate);
		}
		else
		{
			Delegate.ExecuteIfBound(Result);
		}
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FStartStreamingResult>(*this, TEXT("StartStreaming"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::StartStreaming_Internal(const FStartStreamingParameters& Params, FStartStreamingResult& Result)
{
	using ESaveExistsResult = ISaveGameSystem::ESaveExistsResult;

	Result.bRecording = Params.bRecord;

	// Make sure the save game system is available.
	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::StartStreaming: Unable to retrieve save game system")))
	{
		return;
	}

	if (Params.bRecord)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::StartStreaming: Cannot record directly to a save game, use KeepReplay() instead.");
		return;
	}

	const int32 UserIndex = Params.UserIndices.Num() > 0 ? Params.UserIndices[0] : INDEX_NONE;

	// Make sure that the file actually exists.
	const ESaveExistsResult SaveExistsResult = SaveGameSystem->DoesSaveGameExistWithResult(*Params.CustomName, UserIndex);
	if (SaveExistsResult != ESaveExistsResult::OK)
	{
		PopulateStreamingResultFromSaveExistsResult(SaveExistsResult, Result);
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::StartStreaming: Replay does not exist or is invalid.");
		return;
	}

	// Try to load the data.
	TArray<uint8> ReplayData;
	if (!SaveGameSystem->LoadGame(/*bAttemptToUseUI=*/false, *Params.CustomName, UserIndex, ReplayData))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::StartStreaming: Failed to load replay data.");
		return;
	}

	// Copy the data over to the playback slot.
	TSharedPtr<FArchive> CopyAr = CreateLocalFileWriterForOverwrite(GetFullPlaybackName());
	if (CopyAr.IsValid())
	{
		CopyAr->Serialize(ReplayData.GetData(), ReplayData.Num());
	}

	if (!CopyAr.IsValid() || CopyAr->IsError())
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::StartStreaming: Failed to copy replay to local file.");
		return;
	}

	Result.Result = EStreamingOperationResult::Success;
}

void FSaveGameNetworkReplayStreamer::DeleteFinishedStream(const FString& ReplayName, const FDeleteFinishedStreamCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::DeleteFinishedStream(ReplayName, INDEX_NONE, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::DeleteFinishedStream - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FDeleteFinishedStreamResult());
	}
}

void FSaveGameNetworkReplayStreamer::DeleteFinishedStream(const FString& ReplayName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate)
{
	// If we're not handling a SaveFile directly, then just do normal streaming behavior.
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::DeleteFinishedStream(ReplayName, UserIndex, Delegate);
	}
	else if (UserIndex != INDEX_NONE)
	{
		DeleteFinishedStreamSaved(ReplayName, UserIndex, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::DeleteFinishedStream - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FDeleteFinishedStreamResult());
	}
}

void FSaveGameNetworkReplayStreamer::DeleteFinishedStreamSaved(const FString& ReplayName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate) const
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FDeleteFinishedStreamResult>;

	auto AsyncWork = [this, ReplayName, UserIndex]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		DeleteFinishedStream_Internal(ReplayName, UserIndex, *SharedResult.Get());
		return SharedResult;
	};

	auto PostAsyncWork = [Delegate](const FDeleteFinishedStreamResult& Result)
	{
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FDeleteFinishedStreamResult>(*this, TEXT("DeleteFinishedStream"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::DeleteFinishedStream_Internal(const FString& InReplayName, const int32 UserIndex, FDeleteFinishedStreamResult& Result) const
{
	using ESaveExistsResult = ISaveGameSystem::ESaveExistsResult;

	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::DeleteFinishedStream: Unable to retrieve save game systems")))
	{
		return;
	}

	FSaveGameSanitizedNames SanitizedNames;
	if (!StreamNameToSanitizedNames(InReplayName, SanitizedNames))
	{
		return;
	}

	const FString& ReplayMetaName = SanitizedNames.ReplayMetaName;
	const FString& ReplayName = SanitizedNames.ReplayName;

	// Do a quick sanity check to make sure the passed in name is a valid meta or replay name.
	const bool bIsMetaName = ReplayMetaName.Equals(InReplayName);
	const bool bIsReplayName = ReplayName.Equals(InReplayName);
	if (!bIsMetaName && !bIsReplayName)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::DeleteFinishedStream: Invalid Replay name %ls", *InReplayName);
		Result.Result = EStreamingOperationResult::ReplayNotFound;
		return;
	}

	bool bDeletedMeta = SaveGameSystem->DeleteGame(/*bAttemptToUseUI=*/false, *ReplayMetaName, UserIndex);
	bool bDeletedReplay = SaveGameSystem->DeleteGame(/*bAttemptToUseUI=*/false, *ReplayName, UserIndex);

	// If we failed to delete a replay, just make sure they don't actually exist on the system anymore.
	if (!bDeletedMeta)
	{
		bDeletedMeta = (SaveGameSystem->DoesSaveGameExistWithResult(*ReplayMetaName, UserIndex) == ESaveExistsResult::DoesNotExist);
	}
	if (!bDeletedReplay)
	{
		bDeletedReplay = (SaveGameSystem->DoesSaveGameExistWithResult(*ReplayName, UserIndex) == ESaveExistsResult::DoesNotExist);
	}

	if (!bDeletedMeta || !bDeletedReplay)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::DeleteFinishedStream: Unable to delete replay or metadata %ls (meta deleted = %d, replay deleted = %d)", *InReplayName, bDeletedMeta, bDeletedReplay);
		return;
	}

	Result.Result = EStreamingOperationResult::Success;
}

void FSaveGameNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate)
{
	UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay - Invalid UserIndex.");
	Delegate.ExecuteIfBound(FKeepReplayResult());
}

void FSaveGameNetworkReplayStreamer::KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	KeepReplaySaved(ReplayName, bKeep, UserIndex, Delegate);
}

void FSaveGameNetworkReplayStreamer::KeepReplaySaved(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate)
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FKeepReplayResult>;
	using FScopedSaveGameDelegate = SaveGameReplay::FScopedBindExtendedSaveDelegate;

	if (UserIndex == INDEX_NONE)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FKeepReplayResult());
		return;
	}

	auto AsyncWork = [this, ReplayName, bKeep, UserIndex]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		KeepReplay_Internal(ReplayName, bKeep, UserIndex, *SharedResult.Get());
		return SharedResult;
	};

	TSharedPtr<FScopedSaveGameDelegate> ScopedBindExtendedSaveDelegate = MakeShareable<FScopedSaveGameDelegate>(new FScopedSaveGameDelegate(WrapGetSaveGameOption()));
	auto PostAsyncWork = [Delegate, ScopedBindExtendedSaveDelegate](const FKeepReplayResult& Result) mutable
	{
		// Want to release the delegate before sending the result, in case another event is triggered.
		ScopedBindExtendedSaveDelegate.Reset();
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FKeepReplayResult>(*this, TEXT("KeepReplay"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::KeepReplay_Internal(const FString& ReplayName, const bool bKeep, const int32 UserIndex, FKeepReplayResult& Result)
{
	using ESaveExistsResult = ISaveGameSystem::ESaveExistsResult;

	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::KeepReplay: Unable to retrieve save game systems")))
	{
		return;
	}

	if (IsSaveGameFileName(ReplayName))
	{
		// TODO: Maybe we should see whether or not the replay already exists, and change status accordingly?
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Requested to keep an already kept replay %ls", *ReplayName);
		return;
	}

	// Don't go through the process of saving, because the caller told us not to keep it.
	// Note, this is explicitly done after we check the save game system and name to propagate
	// those usage errors early.
	if (!bKeep)
	{
		Result.Result = EStreamingOperationResult::Success;
		Result.NewReplayName = ReplayName;
		return;
	}

	// Make sure the path still exists.
	const FString FullDemoFileName = GetDemoFullFilename(ReplayName);
	if (!FPaths::FileExists(FullDemoFileName))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Requested replay does not exist %ls", *ReplayName);
		Result.Result = EStreamingOperationResult::ReplayNotFound;
		return;
	}

	// Before trying to read anything, make sure there's an open save game slot.
	int32 SaveSlot = 0;
	FSaveGameSanitizedNames SanitizedNames;
	const int32 MaxNumReplays = FConstConsoleVars::GetMaxNumReplaySlots();

	for (; SaveSlot < MaxNumReplays; ++SaveSlot)
	{
		ReplayIndexToSanitizedNames(SaveSlot, SanitizedNames);

		const ESaveExistsResult ReplaySaveFileStatus = SaveGameSystem->DoesSaveGameExistWithResult(*SanitizedNames.ReplayName, UserIndex);
		const ESaveExistsResult MetaSaveFileStatus = SaveGameSystem->DoesSaveGameExistWithResult(*SanitizedNames.ReplayMetaName, UserIndex);

		// At this point, we know at least one (or both) exist.
		// However, either (or both) may be in a bad state. Just skip this for now.
		if (ReplaySaveFileStatus != MetaSaveFileStatus)
		{
			UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Mismatched save file statuses Index=%d ReplayStatus=%d MetaStatus=%d", SaveSlot, static_cast<int32>(ReplaySaveFileStatus), static_cast<int32>(MetaSaveFileStatus));
			continue;
		}
		// We've found an empty slot, stop searching
		else if (ReplaySaveFileStatus == ESaveExistsResult::DoesNotExist)
		{
			break;
		}
	}

	if (SaveSlot == MaxNumReplays)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: No available save slots remain %ls", *ReplayName);
		Result.Result = EStreamingOperationResult::NotEnoughSlots;
		Result.RequiredSpace = MaxNumReplays;
		return;
	}

	// Note, the use of nesting here is done to ensure that the Archive is properly closed **before** sending the delegate.
	// This is done in case users want to perform additional work on the saved file in the delegate.
	{
		const FString& ReplaySaveFileName = SanitizedNames.ReplayName;
		const FString& ReplayMetaSaveFileName = SanitizedNames.ReplayMetaName;

		// Read the file into memory.
		// TODO: When / if the Save Game system supports Archives, we can skip this step.
		TSharedPtr<FArchive> ReplayFileAr = CreateLocalFileReader(FullDemoFileName);
		if (ReplayFileAr.IsValid() && ReplayFileAr->TotalSize() > 0)
		{
			// Read in the Replay and MetaData
			TArray<uint8> ReplayData;
			ReplayData.SetNumUninitialized(IntCastChecked<int32>(ReplayFileAr->TotalSize()));
			ReplayFileAr->Serialize(ReplayData.GetData(), ReplayData.Num());

			// Now, create the Meta Data and save that.
			FSaveGameMetaData MetaData;
			FMemoryReader MetaDataReader(ReplayData);
			if (ReadMetaDataFromLocalStream(MetaDataReader, MetaData))
			{
				// Set the name as the SaveSlot name, and the timestamp as the original file's timestamp.
				MetaData.ReplayName = ReplaySaveFileName;

				// If we don't have a valid timestamp, assume it's the file's timestamp.
				if (MetaData.ReplayInfo.Timestamp == FDateTime::MinValue())
				{
					MetaData.ReplayInfo.Timestamp = IFileManager::Get().GetTimeStamp(*FullDemoFileName);
				}

				TArray<uint8> MetaDataBytes;
				FMemoryWriter MetaDataWriter(MetaDataBytes);
				if (SerializeMetaData(MetaDataWriter, MetaData))
				{
					// The SaveGameSystem will show system dialogs for out of memory when required.
					// However, it's possible that we could run into a situation where either the metadata or the save game fail on OOM.
					// In that case, we would *technically* report a correct save relative to the last save request, but an incorrect total size.

					// Now save the Replay and MetaData.
					TSharedPtr<FSaveGameOptionInfo> OptionInfo = MakeShareable(new FSaveGameOptionInfo());
					WeakOptionInfo = OptionInfo;

					OptionInfo->bIsForRename = false;
					OptionInfo->bIsSavingMetaData = false;
					OptionInfo->ReplayFriendlyName = MetaData.ReplayInfo.FriendlyName;
					OptionInfo->SaveDataSize = ReplayData.Num();

					if (SaveGameSystem->SaveGame(/*bAttemptToUseUI=*/false, *ReplaySaveFileName, UserIndex, ReplayData))
					{

						OptionInfo->bIsSavingMetaData = true;
						OptionInfo->SaveDataSize = MetaDataBytes.Num();

						if (SaveGameSystem->SaveGame(/*bAttemptToUseUI=*/false, *ReplayMetaSaveFileName, UserIndex, MetaDataBytes))
						{
							Result.Result = EStreamingOperationResult::Success;
							Result.NewReplayName = ReplayMetaSaveFileName;
						}
						else
						{
							UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Failed to save replay meta data to slot %ls", *ReplayName);
						}
					}
					else
					{
						UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Failed to save replay to slot %ls", *ReplayName);
					}
				}
				else
				{
					UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Unable to generate meta data %ls", *ReplayName);
				}
			}
			else
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Unable to grab meta data from replay %ls", *ReplayName);
			}
		}
		else
		{
			UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::KeepReplay: Unable to read StreamInfo");
		}
	}
}

void FSaveGameNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName(ReplayName, NewFriendlyName, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RenameReplayFriendlyName - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FRenameReplayResult());
	}
}

void FSaveGameNetworkReplayStreamer::RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::RenameReplayFriendlyName(ReplayName, NewFriendlyName, Delegate);
	}
	else if (UserIndex != INDEX_NONE)
	{
		RenameReplayFriendlyNameSaved(ReplayName, NewFriendlyName, UserIndex, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RenameReplayFriendlyName - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FRenameReplayResult());
	}
}

void FSaveGameNetworkReplayStreamer::RenameReplayFriendlyNameSaved(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) const
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FRenameReplayResult>;
	using FScopedSaveGameDelegate = SaveGameReplay::FScopedBindExtendedSaveDelegate;

	auto AsyncWork = [this, ReplayName, NewFriendlyName, UserIndex]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		RenameReplayFriendlyName_Internal(ReplayName, NewFriendlyName, UserIndex, *SharedResult.Get());
		return SharedResult;
	};

	TSharedPtr<FScopedSaveGameDelegate> ScopedBindExtendedSaveDelegate = MakeShareable<FScopedSaveGameDelegate>(new FScopedSaveGameDelegate(WrapGetSaveGameOption()));
	auto PostAsyncWork = [Delegate, ScopedBindExtendedSaveDelegate](const FRenameReplayResult& Result) mutable
	{
		ScopedBindExtendedSaveDelegate.Reset();
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FRenameReplayResult>(*this, TEXT("RenameReplayFriendlyName"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::RenameReplayFriendlyName_Internal(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, FRenameReplayResult& Result) const
{
	using ESaveExistsResult = ISaveGameSystem::ESaveExistsResult;

	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::RenameReplayFriendlyNameSaved: Unable to retrieve save game systems")))
	{
		return;
	}

	FSaveGameSanitizedNames SanitizedNames;
	if (!StreamNameToSanitizedNames(ReplayName, SanitizedNames))
	{
		return;
	}

	FSaveGameMetaData MetaData;
	if (!ReadMetaDataFromSaveGame(*SaveGameSystem, SanitizedNames, UserIndex, MetaData, Result))
	{
		return;
	}

	MetaData.ReplayInfo.FriendlyName = NewFriendlyName;

	TArray<uint8> MetaDataBytes;
	FMemoryWriter MetaDataWriter(MetaDataBytes);
	if (!SerializeMetaData(MetaDataWriter, MetaData))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RenameReplayFriendlyNameSaved: Failed to write meta data %ls", *ReplayName);
		return;
	}

	TSharedPtr<FSaveGameOptionInfo> OptionInfo = MakeShareable(new FSaveGameOptionInfo());
	WeakOptionInfo = OptionInfo;

	OptionInfo->bIsForRename = true;
	OptionInfo->bIsSavingMetaData = true;
	OptionInfo->SaveDataSize = MetaDataBytes.Num();
	OptionInfo->ReplayFriendlyName = MetaData.ReplayInfo.FriendlyName;

	if (!SaveGameSystem->SaveGame(/*bAttemptToUseUI=*/ false, *SanitizedNames.ReplayMetaName, UserIndex, MetaDataBytes))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RenameReplayFriendlyNameSaved: Failed to save meta data %lss", *ReplayName);
		return;
	}

	Result.Result = EStreamingOperationResult::Success;
}

void FSaveGameNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	RenameReplaySaved(ReplayName, NewName, UserIndex, Delegate);
}

void FSaveGameNetworkReplayStreamer::RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate)
{
	RenameReplaySaved(ReplayName, NewName, INDEX_NONE, Delegate);
}

void FSaveGameNetworkReplayStreamer::RenameReplaySaved(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate)
{
	UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RenameReplay: Is currently unsupported");
	FRenameReplayResult Result;
	Result.Result = EStreamingOperationResult::Unsupported;
	Delegate.ExecuteIfBound(Result);
}

void FSaveGameNetworkReplayStreamer::EnumerateStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate)
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FEnumerateStreamsResult>;

	if (UserIndex == INDEX_NONE)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateStreams - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FEnumerateStreamsResult());
		return;
	}

	auto AsyncWork = [this, ReplayVersion, UserIndex, MetaString, ExtraParms]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		EnumerateStreams_Internal(ReplayVersion, UserIndex, MetaString, ExtraParms, *SharedResult.Get());
		return SharedResult;
	};

	auto PostAsyncWork = [Delegate](const FEnumerateStreamsResult& Result)
	{
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FEnumerateStreamsResult>(*this, TEXT("EnumerateStreams"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::EnumerateStreams_Internal(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, FEnumerateStreamsResult& Result)
{
	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::EnumerateStreams: Unable to retrieve save game systems")))
	{
		const int32 MaxNumReplays = FConstConsoleVars::GetMaxNumReplaySlots();

		Result.Result = EStreamingOperationResult::Success;
		Result.FoundStreams.Reserve(MaxNumReplays);

		// TODO: This could be a lot cleaner if the SaveGameSystem supported enumerate all available save games.
		FStreamingResultBase SaveFileStatus;
		FStreamingResultBase MetaFileStatus;
		FSaveGameSanitizedNames SanitizedNames;

		for (int32 i = 0; i < MaxNumReplays; ++i)
		{
			ReplayIndexToSanitizedNames(i, SanitizedNames);
			const FString& ReplaySaveFileName = SanitizedNames.ReplayName;
			const FString& ReplayMetaSaveFileName = SanitizedNames.ReplayMetaName;

			const auto ReplaySaveFileStatus = SaveGameSystem->DoesSaveGameExistWithResult(*ReplaySaveFileName, UserIndex);
			const auto ReplayMetaSaveFileStatus = SaveGameSystem->DoesSaveGameExistWithResult(*ReplayMetaSaveFileName, UserIndex);

			PopulateStreamingResultFromSaveExistsResult(ReplaySaveFileStatus, SaveFileStatus);
			PopulateStreamingResultFromSaveExistsResult(ReplayMetaSaveFileStatus, MetaFileStatus);

			if (EStreamingOperationResult::ReplayCorrupt == SaveFileStatus.Result || EStreamingOperationResult::ReplayCorrupt == MetaFileStatus.Result)
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateStreams: Found corrupted stream Index=%d", i);

				// Just track the replay name, because we can delete both from that.
				Result.CorruptedStreams.Add(ReplaySaveFileName);
				continue;
			}
			else if (SaveFileStatus.Result != MetaFileStatus.Result)
			{
				UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateStreams: Mismatched result Index=%d ReplayResult=%d MetaResult=%d", i, static_cast<int32>(SaveFileStatus.Result), static_cast<int32>(MetaFileStatus.Result));
				continue;
			}
			// Can ignore cases where the files don't exist.
			else if (EStreamingOperationResult::ReplayNotFound == SaveFileStatus.Result)
			{
				continue;
			}
			// At this point, if the status isn't OK, it's due to an error.
			else if (!SaveFileStatus.WasSuccessful())
			{
				UE_LOGF(LogSaveGameReplay, Log, "FSaveGameNetworkReplayStreamer::EnumerateStreams: Error reading save files Index=%d Status=%d", i, static_cast<int32>(SaveFileStatus.Result));
				continue;
			}

			// Grab the data and sanity check to make sure that the meta file didn't go bad between our last check and now.
			FSaveGameMetaData MetaData;
			if (ReadMetaDataFromSaveGame(*SaveGameSystem, SanitizedNames, UserIndex, MetaData, MetaFileStatus))
			{
				FNetworkReplayStreamInfo& StreamInfo = Result.FoundStreams.Emplace_GetRef();
				PopulateStreamInfoFromMetaData(MetaData, StreamInfo);
				StreamInfo.Name = ReplaySaveFileName;
			}
			else if (EStreamingOperationResult::ReplayCorrupt == MetaFileStatus.Result)
			{
				Result.CorruptedStreams.Add(ReplaySaveFileName);
			}
		}
	}
}

void FSaveGameNetworkReplayStreamer::EnumerateRecentStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate)
{
	FLocalFileNetworkReplayStreamer::EnumerateStreams(ReplayVersion, UserIndex, FString(), TArray<FString>(), Delegate);
}

void FSaveGameNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::EnumerateEvents(ReplayName, Group, INDEX_NONE, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateEvents - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FEnumerateEventsResult());
	}
}

void FSaveGameNetworkReplayStreamer::EnumerateEvents(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::EnumerateEvents(ReplayName, Group, UserIndex, Delegate);
	}
	else if (UserIndex != INDEX_NONE)
	{
		EnumerateEventsSaved(ReplayName, Group, UserIndex, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateEvents - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FEnumerateEventsResult());
	}
}

void FSaveGameNetworkReplayStreamer::EnumerateEventsSaved(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate) const
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FEnumerateEventsResult>;

	auto AsyncWork = [this, ReplayName, Group, UserIndex]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		EnumerateEvents_Internal(ReplayName, Group, UserIndex, *SharedResult.Get());
		return SharedResult;
	};

	auto PostAsyncWork = [Delegate](const FEnumerateEventsResult& Result)
	{
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FEnumerateEventsResult>(*this, TEXT("EnumerateEvents"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::EnumerateEvents_Internal(const FString& ReplayName, const FString& Group, const int32 UserIndex, FEnumerateEventsResult& Result) const
{
	// Note, this may be run Asynchronously, use we can't assume we're on the game thread.
	const FString ConfigFilterGroup = CVarSaveGameFilterEventGroup.GetValueOnAnyThread();
	if (!Group.IsEmpty() && !ConfigFilterGroup.IsEmpty() && Group.Equals(ConfigFilterGroup))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateEvents: Passed in group conflicts with configured value. CVar=%ls | Group=%ls", *ConfigFilterGroup, *Group);
	}

	const FString& UseGroup = Group.IsEmpty() ? ConfigFilterGroup : Group;

	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::EnumerateEventsSaved: Unable to retrieve save game systems")))
	{
		return;
	}

	FSaveGameSanitizedNames SanitizedNames;
	if (!StreamNameToSanitizedNames(ReplayName, SanitizedNames))
	{
		return;
	}

	FSaveGameMetaData MetaData;
	if (ReadMetaDataFromSaveGame(*SaveGameSystem, SanitizedNames, UserIndex, MetaData, Result))
	{
		Result.Result = EStreamingOperationResult::Success;

		if (UseGroup.IsEmpty())
		{
			Result.ReplayEventList = MoveTemp(MetaData.VersionedInfo.Events);
		}
		else
		{
			TArray<FReplayEventListItem>& InReplayEvents = MetaData.VersionedInfo.Events.ReplayEvents;
			TArray<FReplayEventListItem>& OutReplayEvents = Result.ReplayEventList.ReplayEvents;

			for (FReplayEventListItem& ReplayEvent : InReplayEvents)
			{
				if (UseGroup.Equals(ReplayEvent.Group))
				{
					OutReplayEvents.Add(MoveTemp(ReplayEvent));
				}
			}
		}
	}
}

void FSaveGameNetworkReplayStreamer::RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	// Once a replay is saved, it's stream name will have changed.
	// Therefore, the name encoded in the event won't be helpful.
	// However, if the replay hasn't been saved yet, it still may be findable with it's original name.
	UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RequestEventData: No replay name available, defaulting to FLocalFileStreamFArchive::RequestEventData");
	FLocalFileNetworkReplayStreamer::RequestEventData(EventID, Delegate);
}

void FSaveGameNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::RequestEventData(ReplayName, EventID, INDEX_NONE, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RequestEventData - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FRequestEventDataResult());
	}
}

void FSaveGameNetworkReplayStreamer::RequestEventData(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::RequestEventData(ReplayName, EventID, UserIndex, Delegate);
	}
	else if (UserIndex != INDEX_NONE)
	{
		RequestEventDataSaved(ReplayName, EventID, UserIndex, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RequestEventData - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FRequestEventDataResult());
	}
}

void FSaveGameNetworkReplayStreamer::RequestEventDataSaved(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate)
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FRequestEventDataResult>;

	auto AsyncWork = [this, ReplayName, EventID, UserIndex]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		RequestEventData_Internal(ReplayName, EventID, UserIndex, *SharedResult.Get());
		return SharedResult;
	};

	auto PostAsyncWork = [Delegate](const FRequestEventDataResult& Result)
	{
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FRequestEventDataResult>(*this, TEXT("RequestEventData"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::RequestEventData_Internal(const FString& ReplayName, const FString& EventID, const int32 UserIndex, FRequestEventDataResult& Result)
{
	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::RequestEventDataSaved: Unable to retrieve save game systems")))
	{
		return;
	}

	FSaveGameSanitizedNames SanitizedNames;
	if (!StreamNameToSanitizedNames(ReplayName, SanitizedNames))
	{
		return;
	}

	FSaveGameMetaData MetaData;
	if (ReadMetaDataFromSaveGame(*SaveGameSystem, SanitizedNames, UserIndex, MetaData, Result))
	{
		for (int32 i = 0; i < MetaData.VersionedInfo.Events.ReplayEvents.Num(); ++i)
		{
			if (MetaData.VersionedInfo.Events.ReplayEvents[i].ID.Equals(EventID))
			{
				Result.Result = EStreamingOperationResult::Success;
				Result.ReplayEventListItem = MoveTemp(MetaData.VersionedInfo.EventData[i]);
				break;
			}
		}

		// we didn't find the event
		if (Result.Result != EStreamingOperationResult::Success)
		{
			Result.Result = EStreamingOperationResult::EventNotFound;
		}
	}
}

void FSaveGameNetworkReplayStreamer::RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RequestEventGroupData: No replay name available, defaulting to FLocalFileStreamFArchive::RequestEventData");
	FLocalFileNetworkReplayStreamer::RequestEventGroupData(Group, Delegate);
}

void FSaveGameNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::RequestEventGroupData(ReplayName, Group, INDEX_NONE, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RequestEventGroupData - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FRequestEventGroupDataResult());
	}
}

void FSaveGameNetworkReplayStreamer::RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	if (!IsSaveGameFileName(ReplayName))
	{
		FLocalFileNetworkReplayStreamer::RequestEventGroupData(ReplayName, Group, UserIndex, Delegate);
	}
	else if (UserIndex != INDEX_NONE)
	{
		RequestEventGroupDataSaved(ReplayName, Group, UserIndex, Delegate);
	}
	else
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::RequestEventGroupData - Invalid UserIndex.");
		Delegate.ExecuteIfBound(FRequestEventGroupDataResult());
	}
}

void FSaveGameNetworkReplayStreamer::RequestEventGroupDataSaved(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate)
{
	using TAsyncTypes = SaveGameReplay::TAsyncTypes<FRequestEventGroupDataResult>;

	auto AsyncWork = [this, ReplayName, Group, UserIndex]()
	{
		auto SharedResult = TAsyncTypes::MakeSharedResult();
		RequestEventGroupData_Internal(ReplayName, Group, UserIndex, *SharedResult.Get());
		return SharedResult;
	};

	auto PostAsyncWork = [Delegate](const FRequestEventGroupDataResult& Result)
	{
		Delegate.ExecuteIfBound(Result);
	};

	SaveGameReplay::FAsyncTaskManager::Get().StartTask<FRequestEventGroupDataResult>(*this, TEXT("RequestEventGroupData"), AsyncWork, PostAsyncWork);
}

void FSaveGameNetworkReplayStreamer::RequestEventGroupData_Internal(const FString& ReplayName, const FString& Group, const int32 UserIndex, FRequestEventGroupDataResult& Result)
{
	ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
	if (!ensureMsgf(SaveGameSystem, TEXT("FSaveGameNetworkReplayStreamer::RequestEventGroupDataSaved: Unable to retrieve save game systems")))
	{
		return;
	}

	FSaveGameSanitizedNames SanitizedNames;
	if (!StreamNameToSanitizedNames(ReplayName, SanitizedNames))
	{
		return;
	}

	FSaveGameMetaData MetaData;
	if (ReadMetaDataFromSaveGame(*SaveGameSystem, SanitizedNames, UserIndex, MetaData, Result))
	{
		for (int32 i = 0; i < MetaData.VersionedInfo.Events.ReplayEvents.Num(); ++i)
		{
			const FReplayEventListItem& EventInfo = MetaData.VersionedInfo.Events.ReplayEvents[i];

			if (EventInfo.Group == Group)
			{
				Result.Result = EStreamingOperationResult::Success;
				
				FReplayEventListItem& EventItem = Result.ReplayEventListItems.AddDefaulted_GetRef();
				EventItem.ID = EventInfo.ID;
				EventItem.Group = EventInfo.Group;
				EventItem.Metadata = EventInfo.Metadata;
				EventItem.Time1 = EventInfo.Time1;
				EventItem.Time2 = EventInfo.Time2;

				FRequestEventDataResult& EventDataResult = Result.ReplayEventListResults.AddDefaulted_GetRef();
				EventDataResult.Result = EStreamingOperationResult::Success;
				EventDataResult.ReplayEventListItem = MoveTemp(MetaData.VersionedInfo.EventData[i]);
			}
		}

		// we didn't find the event group
		if (Result.Result != EStreamingOperationResult::Success)
		{
			Result.Result = EStreamingOperationResult::EventNotFound;
		}
	}
}

bool FSaveGameNetworkReplayStreamer::ReadMetaDataFromLocalStream(FArchive& StreamArchive, FSaveGameMetaData& OutMetaData) const
{
	check(StreamArchive.IsLoading());

	FLocalFileReplayInfo FileReplayInfo;
	if (!ReadReplayInfo(StreamArchive, FileReplayInfo, EReadReplayInfoFlags::None) || !FileReplayInfo.bIsValid)
	{
		return false;
	}

	const FString FilterGroup = CVarSaveGameFilterEventGroup.GetValueOnAnyThread();
	const bool bShouldFilter = !FilterGroup.IsEmpty();

	TArray<FReplayEventListItem> EventList;
	TArray<TArray<uint8>> EventData;

	for (int32 i = 0; i < FileReplayInfo.Events.Num() && !StreamArchive.IsError(); ++i)
	{
		FLocalFileEventInfo& LocalEvent = FileReplayInfo.Events[i];
		if (bShouldFilter && !LocalEvent.Group.Equals(FilterGroup))
		{
			continue;
		}

		FReplayEventListItem ReplayEvent;
		TArray<uint8> ReplayEventData;

		ReplayEvent.ID = MoveTemp(LocalEvent.Id);
		ReplayEvent.Group = MoveTemp(LocalEvent.Group);
		ReplayEvent.Metadata = MoveTemp(LocalEvent.Metadata);
		ReplayEvent.Time1 = LocalEvent.Time1;
		ReplayEvent.Time2 = LocalEvent.Time2;

		if (LocalEvent.SizeInBytes > 0)
		{
			if (LocalEvent.SizeInBytes > SaveGameReplay::MaxEventSize)
			{
				UE_LOGF(LogSaveGameReplay, Error, "ReadMetaDataFromLocalStream: Event SizeInBytes > MaxEventSize (%d / %d)", LocalEvent.SizeInBytes, SaveGameReplay::MaxEventSize);
				return false;
			}

			ReplayEventData.SetNumUninitialized(LocalEvent.SizeInBytes);
			StreamArchive.Seek(LocalEvent.EventDataOffset);
			StreamArchive.Serialize(ReplayEventData.GetData(), ReplayEventData.Num());

			if (FileReplayInfo.bEncrypted)
			{
				if (SupportsEncryption())
				{
					TArray<uint8> PlaintextData;
										
					if (!DecryptBuffer(ReplayEventData, PlaintextData, FileReplayInfo.EncryptionKey))
					{
						UE_LOGF(LogSaveGameReplay, Error, "FSaveGameNetworkReplayStreamer::ReadMetaDataFromLocalStream. DecryptBuffer failed.");
						return false;
					}

					ReplayEventData = MoveTemp(PlaintextData);
				}
				else
				{
					UE_LOGF(LogSaveGameReplay, Error, "FSaveGameNetworkReplayStreamer::ReadMetaDataFromLocalStream. Encrypted event but streamer does not support encryption.");
					return false;
				}
			}
		}

		EventList.Add(MoveTemp(ReplayEvent));
		EventData.Add(MoveTemp(ReplayEventData));
	}

	const bool bSuccess = !StreamArchive.IsError();

	if (bSuccess)
	{
		OutMetaData.VersionedInfo.FileVersion = SaveGameReplay::HISTORY_LATEST;
		OutMetaData.VersionedInfo.EventData = MoveTemp(EventData);
		OutMetaData.VersionedInfo.Events.ReplayEvents = MoveTemp(EventList);

		OutMetaData.ReplayInfo = MoveTemp(FileReplayInfo);
	}

	return bSuccess;
}

bool FSaveGameNetworkReplayStreamer::ReadMetaDataFromSaveGame(class ISaveGameSystem& SaveGameSystem, const FSaveGameSanitizedNames& SanitizedNames, const int32 UserIndex, FSaveGameMetaData& OutMetaData, FStreamingResultBase& OutResult) const
{
	using ESaveExistsResult = ISaveGameSystem::ESaveExistsResult;

	const ESaveExistsResult ReplayMetaSaveFileStatus = SaveGameSystem.DoesSaveGameExistWithResult(*SanitizedNames.ReplayMetaName, UserIndex);
	
	if (ESaveExistsResult::OK != ReplayMetaSaveFileStatus)
	{
		PopulateStreamingResultFromSaveExistsResult(ReplayMetaSaveFileStatus, OutResult);
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::ReadMetaDataFromSaveGame: Replay does not exist or is invalid.");
		return false;
	}

	TArray<uint8> MetaDataBytes;
	if (!SaveGameSystem.LoadGame(/** bAttemptToUseUI= */ false, *SanitizedNames.ReplayMetaName, UserIndex, MetaDataBytes))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::ReadMetaDataFromSaveGame: Failed to load replay.");
		return false;
	}

	FMemoryReader MetaDataReader(MetaDataBytes);
	if (!SerializeMetaData(MetaDataReader, OutMetaData))
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::EnumerateEvents: Failed to read meta data.");
		return false;
	}

	return true;
}

void FSaveGameNetworkReplayStreamer::PopulateStreamInfoFromMetaData(const FSaveGameMetaData& MetaData, FNetworkReplayStreamInfo& OutStreamInfo) const
{
	OutStreamInfo.Name = MetaData.ReplayName;
	OutStreamInfo.Timestamp = MetaData.ReplayInfo.Timestamp;
	OutStreamInfo.LengthInMS = MetaData.ReplayInfo.LengthInMS;
	OutStreamInfo.FriendlyName = MetaData.ReplayInfo.FriendlyName;
	OutStreamInfo.SizeInBytes = MetaData.ReplayInfo.TotalDataSizeInBytes;
	OutStreamInfo.Changelist = MetaData.ReplayInfo.Changelist;

	OutStreamInfo.bIsLive = false;
	OutStreamInfo.NumViewers = 0;
}

bool FSaveGameNetworkReplayStreamer::SerializeMetaData(FArchive& Archive, FSaveGameMetaData& MetaData) const
{
	if (Archive.IsSaving())
	{
		// Because we store meta data separately, we don't care about the replay version, and can
		// safely change the name length.
		FixupFriendlyNameLength(MetaData.ReplayInfo.FriendlyName, MetaData.ReplayInfo.FriendlyName);
	}

	Archive << MetaData.ReplayName;
	Archive << MetaData.ReplayInfo.FriendlyName;
	Archive << MetaData.ReplayInfo.Timestamp;
	Archive << MetaData.ReplayInfo.TotalDataSizeInBytes;
	Archive << MetaData.ReplayInfo.LengthInMS;
	Archive << MetaData.ReplayInfo.Changelist;

	if (Archive.IsLoading())
	{
		MetaData.ReplayInfo.FriendlyName.TrimStartAndEndInline();
	}

	return !Archive.IsError() && SerializeVersionedMetaData(Archive, MetaData);
}

bool FSaveGameNetworkReplayStreamer::SerializeVersionedMetaData(FArchive& Archive, FSaveGameMetaData& MetaData) const
{
	if ((Archive.IsLoading() && Archive.AtEnd()) ||
		(Archive.IsSaving() && SaveGameReplay::HISTORY_INITIAL == MetaData.VersionedInfo.FileVersion))
	{
		MetaData.VersionedInfo.FileVersion = SaveGameReplay::HISTORY_INITIAL;
		return true;
	}

	Archive << MetaData.VersionedInfo.FileVersion;

	if (MetaData.VersionedInfo.FileVersion >= SaveGameReplay::HISTORY_EVENTS)
	{
		TArray<FReplayEventListItem>& Events = MetaData.VersionedInfo.Events.ReplayEvents;
		TArray<TArray<uint8>>& EventData = MetaData.VersionedInfo.EventData;

		int32 NumEvents = Events.Num();
		Archive << NumEvents;

		if (NumEvents > SaveGameReplay::MaxEvents)
		{
			UE_LOGF(LogSaveGameReplay, Error, "SerializeVersionedMetaData: NumEvents > MaxEvents (%d / %d)", NumEvents, SaveGameReplay::MaxEvents);
			Archive.SetError();
			return false;
		}

		Events.SetNum(NumEvents);
		EventData.SetNum(NumEvents);

		for (int32 i = 0; i < NumEvents && !Archive.IsError(); ++i)
		{
			FReplayEventListItem& ReplayEvent = Events[i];
			TArray<uint8>& ReplayEventData = EventData[i];

			Archive << ReplayEvent.Group;
			Archive << ReplayEvent.ID;
			Archive << ReplayEvent.Metadata;
			Archive << ReplayEvent.Time1;
			Archive << ReplayEvent.Time2;

			int32 EventDataSize = ReplayEventData.Num();
			Archive << EventDataSize;

			if (EventDataSize > 0)
			{
				if (EventDataSize > SaveGameReplay::MaxEventSize)
				{
					UE_LOGF(LogSaveGameReplay, Error, "SerializeVersionedMetaData: EventDataSize > MaxEventSize (%d / %d)", EventDataSize, SaveGameReplay::MaxEventSize);
					Archive.SetError();
					return false;
				}

				ReplayEventData.SetNum(EventDataSize);
				Archive.Serialize(ReplayEventData.GetData(), ReplayEventData.Num());
			}
		}
	}

	return !Archive.IsError();
}

bool FSaveGameNetworkReplayStreamer::IsSaveGameFileName(const FString& ReplayName) const
{
	return SaveGameReplay::IsSaveGameFileName(ReplayName);
}

bool FSaveGameNetworkReplayStreamer::StreamNameToSanitizedNames(const FString& StreamName, FSaveGameSanitizedNames& OutSanitizedNames) const
{
	const int32 ReplayIndex = GetReplayIndexFromName(StreamName);
	if (INDEX_NONE == ReplayIndex)
	{
		UE_LOGF(LogSaveGameReplay, Warning, "FSaveGameNetworkReplayStreamer::StreamNameToSanitizedNames: Failed to parse replay index from name %ls", *StreamName);
		return false;
	}

	ReplayIndexToSanitizedNames(ReplayIndex, OutSanitizedNames);
	return true;
}

void FSaveGameNetworkReplayStreamer::ReplayIndexToSanitizedNames(const int32 ReplayIndex, FSaveGameSanitizedNames& OutSanitizedNames) const
{
	OutSanitizedNames.ReplayIndex = ReplayIndex;
	OutSanitizedNames.ReplayName = FString::Printf(ReplaySaveFileFormat, ReplayIndex);
	OutSanitizedNames.ReplayMetaName = FString::Printf(ReplayMetaSaveFileFormat, ReplayIndex);
}

int32 FSaveGameNetworkReplayStreamer::GetReplayIndexFromName(const FString& ReplayName) const
{
	return SaveGameReplay::GetReplayIndexFromName(ReplayName);
}

FString FSaveGameNetworkReplayStreamer::GetFullPlaybackName() const
{
	// Note, we don't want this file to get enumerated later, otherwise it may look like we have an unsaved replay.
	// So, we'll use a subdirectory for the playback file.
	// This should work because the LocalFileStreamer only enumerates the top level directory.
	// TODO: Maybe we could just put this into a legitimate temporary directory.
	return GetDemoFullFilename(GetLocalPlaybackName());
}

FString FSaveGameNetworkReplayStreamer::GetLocalPlaybackName() const
{
	return FPaths::Combine(GetTempDemoRelativeSavePath(), PlaybackReplayName);
}

TArrayView<const FString> FSaveGameNetworkReplayStreamer::GetAdditionalRelativeDemoPaths() const
{
	return MakeArrayView(&GetTempDemoRelativeSavePath(), 1);
}

const FString& FSaveGameNetworkReplayStreamer::GetDefaultDemoSavePath()
{
	static const FString DefaultDemoSavePath = FPaths::Combine(*FPaths::ProjectPersistentDownloadDir(), TEXT("Demos/"));
	return DefaultDemoSavePath;
}

const FString& FSaveGameNetworkReplayStreamer::GetTempDemoRelativeSavePath()
{
	// Allow for cleanup of the local playback.replay file if needed
	static const FString TempDemoSavePath("Temp/");
	return TempDemoSavePath;
}

const FString& FSaveGameNetworkReplayStreamer::GetDefaultPlaybackName()
{
	static const FString DefaultPlaybackName("Playback");
	return DefaultPlaybackName;
}

TFunction<bool(const TCHAR*, EGameDelegates_SaveGame, FString&)> FSaveGameNetworkReplayStreamer::WrapGetSaveGameOption() const
{
	return [this](const TCHAR* FileName, EGameDelegates_SaveGame Option, FString& OptionValue)
	{
		check(WeakOptionInfo.IsValid());

		TSharedPtr<FSaveGameOptionInfo> OptionInfo = WeakOptionInfo.Pin();
		OptionInfo->Option = Option;
		OptionInfo->ReplaySaveName = FString(FileName);
		return GetSaveGameOption(*OptionInfo.Get(), OptionValue);
	};
}

IMPLEMENT_MODULE(FSaveGameNetworkReplayStreamingFactory, SaveGameNetworkReplayStreaming)

void FSaveGameNetworkReplayStreamingFactory::StartupModule()
{
	FSaveGameNetworkReplayStreamer::CleanUpOldReplays(
		FSaveGameNetworkReplayStreamer::GetDefaultDemoSavePath(),
		MakeArrayView(&FSaveGameNetworkReplayStreamer::GetTempDemoRelativeSavePath(), 1));
}

TSharedPtr<INetworkReplayStreamer> FSaveGameNetworkReplayStreamingFactory::CreateReplayStreamer()
{
	if (IPlatformFeaturesModule::Get().GetSaveGameSystem() == nullptr)
	{
		UE_LOGF(LogSaveGameReplay, Error, "FSaveGameNetworkReplayStreamingFactory: Unable to get SaveGameSystem.");
		return nullptr;
	}

	TSharedPtr<FSaveGameNetworkReplayStreamer> Streamer = MakeShared<FSaveGameNetworkReplayStreamer>();
	LocalFileStreamers.Add(Streamer);
	return Streamer;
}
