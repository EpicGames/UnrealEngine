// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "NetworkReplayStreaming.h"
#include "LocalFileNetworkReplayStreaming.h"

enum class EGameDelegates_SaveGame : short;

/**
 * Local file streamer that supports playback/recording to files on disk, and transferring replays to and from SaveGame slots.
 *
 * EnumerateStreams may be used to list all available replays that are in SaveGame slots.
 * The Name member in any FNetworkReplayStreamInfo returned will be the SaveGame slot where the replay lives.
 *
 * EnumerateRecentStreams may be used to list all available replays that are not in SaveGame slots.
 * The Name member in any FNetworkReplayStreamInfo returned will be the relative path where the replay lives.
 *
 * StartStreaming can be used to play replays both in and not in SaveGame slots.
 * StartStreaming does not automatically put a replay in a SaveGame slot.
 *
 * KeepReplay can be used to move a non SaveGame slot replay into a SaveGame slot. The original replay is left untouched.
 *
 * DeleteFinishedStream can be used to delete replays both in and not in SaveGame slots.
 *
 * Only one Save Game operation is permitted to occur at a single time (even across Streamers).
 *
 * TODO: Proper handling of UserIndex.
 */
class FSaveGameNetworkReplayStreamer : public FLocalFileNetworkReplayStreamer
{
public:

	SAVEGAMENETWORKREPLAYSTREAMING_API FSaveGameNetworkReplayStreamer();
	SAVEGAMENETWORKREPLAYSTREAMING_API FSaveGameNetworkReplayStreamer(const FString& DemoSavePath, const FString& PlaybackReplayName);

	/** INetworkReplayStreamer implementation */
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void StartStreaming(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void DeleteFinishedStream(const FString& ReplayName, const FDeleteFinishedStreamCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void DeleteFinishedStream(const FString& StreamName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void EnumerateStreams(const FNetworkReplayVersion& InReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, const FEnumerateStreamsCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void EnumerateRecentStreams(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FEnumerateStreamsCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const FKeepReplayCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void KeepReplay(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const FRenameReplayCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RenameReplayFriendlyName(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const FRenameReplayCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RenameReplay(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate) override;

	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void EnumerateEvents(const FString& ReplayName, const FString& Group, const FEnumerateEventsCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void EnumerateEvents( const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RequestEventData(const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RequestEventData(const FString& ReplayName, const FString& EventID, const FRequestEventDataCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RequestEventData(const FString& ReplayName, const FString& EventId, const int32 UserIndex, const FRequestEventDataCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RequestEventGroupData(const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const FRequestEventGroupDataCallback& Delegate) override;
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void RequestEventGroupData(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate) override;
protected:

	SAVEGAMENETWORKREPLAYSTREAMING_API void StartStreamingSaved(const FStartStreamingParameters& Params, const FStartStreamingCallback& Delegate);
	SAVEGAMENETWORKREPLAYSTREAMING_API void DeleteFinishedStreamSaved(const FString& ReplayName, const int32 UserIndex, const FDeleteFinishedStreamCallback& Delegate) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void KeepReplaySaved(const FString& ReplayName, const bool bKeep, const int32 UserIndex, const FKeepReplayCallback& Result);
	SAVEGAMENETWORKREPLAYSTREAMING_API void RenameReplayFriendlyNameSaved(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, const FRenameReplayCallback& Delegate) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void RenameReplaySaved(const FString& ReplayName, const FString& NewName, const int32 UserIndex, const FRenameReplayCallback& Delegate);
	SAVEGAMENETWORKREPLAYSTREAMING_API void EnumerateEventsSaved(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FEnumerateEventsCallback& Delegate) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void RequestEventDataSaved(const FString& ReplayName, const FString& EventID, const int32 UserIndex, const FRequestEventDataCallback& Delegate);
	SAVEGAMENETWORKREPLAYSTREAMING_API void RequestEventGroupDataSaved(const FString& ReplayName, const FString& Group, const int32 UserIndex, const FRequestEventGroupDataCallback& Delegate);

	SAVEGAMENETWORKREPLAYSTREAMING_API void StartStreaming_Internal(const FStartStreamingParameters& Params, FStartStreamingResult& Result);
	SAVEGAMENETWORKREPLAYSTREAMING_API void DeleteFinishedStream_Internal(const FString& ReplayName, const int32 UserIndex, FDeleteFinishedStreamResult& Result) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void EnumerateStreams_Internal(const FNetworkReplayVersion& ReplayVersion, const int32 UserIndex, const FString& MetaString, const TArray< FString >& ExtraParms, FEnumerateStreamsResult& Result);
	SAVEGAMENETWORKREPLAYSTREAMING_API void KeepReplay_Internal(const FString& ReplayName, const bool bKeep, const int32 UserIndex, FKeepReplayResult& Result);
	SAVEGAMENETWORKREPLAYSTREAMING_API void RenameReplayFriendlyName_Internal(const FString& ReplayName, const FString& NewFriendlyName, const int32 UserIndex, FRenameReplayResult& Result) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void EnumerateEvents_Internal(const FString& ReplayName, const FString& Group, const int32 UserIndex, FEnumerateEventsResult& Result) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void RequestEventData_Internal(const FString& ReplayName, const FString& EventID, const int32 UserIndex, FRequestEventDataResult& Result);
	SAVEGAMENETWORKREPLAYSTREAMING_API void RequestEventGroupData_Internal(const FString& ReplayName, const FString& Group, const int32 UserIndex, FRequestEventGroupDataResult& Result);

	struct FSaveGameReplayVersionedInfo
	{
		// Save game file version.
		uint32 FileVersion;

		// Events that are serialized in the header.
		FReplayEventList Events;

		// Actual event data. Indices correlate to Event index.
		TArray<TArray<uint8>> EventData;
	};

	struct FSaveGameMetaData
	{
		FString ReplayName;

		FLocalFileReplayInfo ReplayInfo;

		FSaveGameReplayVersionedInfo VersionedInfo;
	};

	struct FSaveGameSanitizedNames
	{
		FString ReplayMetaName;
		FString ReplayName;
		int32 ReplayIndex;
	};

	SAVEGAMENETWORKREPLAYSTREAMING_API bool StreamNameToSanitizedNames(const FString& StreamName, FSaveGameSanitizedNames& OutSanitizedNames) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void ReplayIndexToSanitizedNames(const int32 ReplayIndex, FSaveGameSanitizedNames& OutSanitizedNames) const;

	SAVEGAMENETWORKREPLAYSTREAMING_API bool ReadMetaDataFromLocalStream(FArchive& Archive, FSaveGameMetaData& OutMetaData) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API bool ReadMetaDataFromSaveGame(class ISaveGameSystem& SaveGameSystem, const FSaveGameSanitizedNames& SanitizedNames, const int32 UserIndex, FSaveGameMetaData& OutMetaData, FStreamingResultBase& OutResult) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API void PopulateStreamInfoFromMetaData(const FSaveGameMetaData& MetaData, FNetworkReplayStreamInfo& OutStreamInfo) const;

	SAVEGAMENETWORKREPLAYSTREAMING_API bool SerializeMetaData(FArchive& Archive, FSaveGameMetaData& MetaData) const;
	SAVEGAMENETWORKREPLAYSTREAMING_API bool SerializeVersionedMetaData(FArchive& Archive, FSaveGameMetaData& MetaData) const;

	// Returns whether the input name corresponds to a save game.
	SAVEGAMENETWORKREPLAYSTREAMING_API bool IsSaveGameFileName(const FString& ReplayName) const;

	SAVEGAMENETWORKREPLAYSTREAMING_API int32 GetReplayIndexFromName(const FString& ReplayName) const;

	SAVEGAMENETWORKREPLAYSTREAMING_API FString GetFullPlaybackName() const;
	SAVEGAMENETWORKREPLAYSTREAMING_API FString GetLocalPlaybackName() const;

	SAVEGAMENETWORKREPLAYSTREAMING_API virtual TArrayView<const FString> GetAdditionalRelativeDemoPaths() const override;

	struct FSaveGameOptionInfo
	{
		EGameDelegates_SaveGame Option;
		bool bIsForRename = false;
		bool bIsSavingMetaData = false;
		int32 SaveDataSize = INDEX_NONE;

		FString ReplayFriendlyName;
		FString ReplaySaveName;
	};

	/**
	 * Called during KeepReplay to get options when saving a replay.
	 * Note, this may be called off the GameThread and may not be called on every platform.
	 * @see FGameDelegates::GetExtendedSaveGameInfoDelegate
	 *
	 * @param OptionInfo	Info struct that describes the requested option and current streamer status.
	 * @param OptionValue	(Out) The desired value for the option.
	 *
	 * @return True if this event was handled. False if it should be passed to the original delegate (
	 */
	virtual bool GetSaveGameOption(const FSaveGameOptionInfo& OptionInfo, FString& OptionValue) const { return false; }


	// Special replay name that will be used when copying over SaveGame replays for playback.
	const FString PlaybackReplayName;

	static SAVEGAMENETWORKREPLAYSTREAMING_API const FString& GetDefaultDemoSavePath();
	static SAVEGAMENETWORKREPLAYSTREAMING_API const FString& GetTempDemoRelativeSavePath();
	static SAVEGAMENETWORKREPLAYSTREAMING_API const FString& GetDefaultPlaybackName();

private:

	SAVEGAMENETWORKREPLAYSTREAMING_API TFunction<bool(const TCHAR*, EGameDelegates_SaveGame, FString&)> WrapGetSaveGameOption() const;

	// Although this isn't used on the GameThread, it should only be created / destroyed
	// from the same thread. Therefore, no need to make it thread safe (for now).
	mutable TWeakPtr<FSaveGameOptionInfo> WeakOptionInfo;

	friend class FSaveGameNetworkReplayStreamingFactory;
};

class FSaveGameNetworkReplayStreamingFactory : public FLocalFileNetworkReplayStreamingFactory
{
public:
	SAVEGAMENETWORKREPLAYSTREAMING_API virtual void StartupModule() override;

	SAVEGAMENETWORKREPLAYSTREAMING_API virtual TSharedPtr<INetworkReplayStreamer> CreateReplayStreamer() override;
};
