// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SaveGameNetworkReplayStreaming : ModuleRules
	{
		public SaveGameNetworkReplayStreaming(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"NetworkReplayStreaming",
					"LocalFileNetworkReplayStreaming"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"Json"
				}
			);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
